{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Cascade Editor",
	"Description" : "Editor for (legacy) Cascade effect systems. Cascade assets can still run in a game with this plugin disabled.",
	"Category" : "FX",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Plugins": [],
	"Modules" :
	[
		{
			"Name" : "Cascade",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	] 
}
