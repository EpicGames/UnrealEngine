// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Cascade : ModuleRules
{
	public Cascade(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"DistCurveEditor",
				"UnrealEd",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
                "AppFramework",
				"ContentBrowser",
				"Core",
				"CoreUObject",
                "InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				"DistCurveEditor",
				"EditorFramework",
				"ApplicationCore",
				"UnrealEd",
				"ToolMenus",
				"RHI",
				"RenderCore",
				"PhysicsCore"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"PropertyEditor"
			}
		);
	}
}
