// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CascadeConfiguration
//
// Settings for Cascade that users are not allowed to alter.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CascadeConfiguration.generated.h"

/** Module-to-TypeData mapping helper. */
USTRUCT()
struct FModuleMenuMapper
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString ObjName;

	UPROPERTY()
	TArray<FString> InvalidObjNames;

};

UCLASS(hidecategories=Object, config=Editor)
class UCascadeConfiguration : public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 *	TypeData-to-base module mappings.
	 *	These will disallow complete 'sub-menus' depending on the TypeData utilized.
	 */
	UPROPERTY(EditAnywhere, config, Category=Configure)
	TArray<struct FModuleMenuMapper> ModuleMenu_TypeDataToBaseModuleRejections;

	/** Module-to-TypeData mappings. */
	UPROPERTY(EditAnywhere, config, Category=Configure)
	TArray<struct FModuleMenuMapper> ModuleMenu_TypeDataToSpecificModuleRejections;

	/** Modules that Cascade should ignore in the menu system. */
	UPROPERTY(EditAnywhere, config, Category=Configure)
	TArray<FString> ModuleMenu_ModuleRejections;

	/** Returns true if the given module class name is valid for the type data class name. */
	bool IsModuleTypeValid(FName TypeDataName, FName ModuleName);

private:
	/** Cache module rejections for fast runtime lookups. */
	void CacheModuleRejections();

	/** If a module class name is in this list it should be rejected. */
	TSet<FName> ModuleRejections;

	/** If a module class name is in the set associated with a type data class name it should be rejected. */
	TMap<FName,TSet<FName> > TypeDataModuleRejections;
};

