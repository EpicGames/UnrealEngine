// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Particles/ParticleSystemComponent.h"
#include "CascadeParticleSystemComponent.generated.h"

/** 
 * Used to provide an extended particle system component to allow collision to function in the preview window.
 */
UCLASS(hidecategories=Object, hidecategories=Physics, hidecategories=Collision, editinlinenew)
class UCascadeParticleSystemComponent : public UParticleSystemComponent
{
	GENERATED_UCLASS_BODY()


public:	
	class FCascadeEdPreviewViewportClient*	CascadePreviewViewportPtr;

	//~ Begin UActorComponent Interface
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface

	/**
	 * Updates time dependent state for this component, called from within Cascade.
	 * Requires component to be registered
	 * @param DeltaTime - The time since the last tick.
	 * @param TickType - The kind of tick this is
	 */
	void CascadeTickComponent(float DeltaTime, enum ELevelTick TickType);

	// Collision Handling...
	virtual bool ParticleLineCheck(FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& Extent, const FCollisionObjectQueryParams&) override;

	virtual bool Editor_CanBeTickManaged()const { return false; }

protected:
	virtual void UpdateLODInformation() override;
};

