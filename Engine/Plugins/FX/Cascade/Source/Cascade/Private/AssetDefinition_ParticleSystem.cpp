// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ParticleSystem.h"

#include "CascadeModule.h"
#include "ContentBrowserMenuContexts.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/FeedbackContext.h"
#include "Particles/ParticleEmitter.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_ParticleSystem"

EAssetCommandResult UAssetDefinition_ParticleSystem::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	const TArray<UObject*>& Objects = OpenArgs.LoadObjects<UObject>();
	for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		UParticleSystem* ParticleSystem = Cast<UParticleSystem>(*ObjIt);
		if (ParticleSystem != nullptr)
		{
			ICascadeModule* CascadeModule = &FModuleManager::LoadModuleChecked<ICascadeModule>("Cascade");
			CascadeModule->CreateCascade(Mode, OpenArgs.ToolkitHost, ParticleSystem);
		}
	}
	return EAssetCommandResult::Handled;
}

namespace MenuExtension::ParticleSystem
{
	void ExecuteCopyParameters(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		const TArray<UParticleSystem*> Objects = CBContext->LoadSelectedObjects<UParticleSystem>();

		FString ClipboardString = TEXT("");
		for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			if (UParticleSystem* Object = *ObjIt)
			{
				TArray<TArray<FString>> ParticleSysParamList;
				TArray<TArray<FString>> ParticleParameterList;
				Object->GetParametersUtilized(ParticleSysParamList, ParticleParameterList);

				ClipboardString += FString::Printf(TEXT("ParticleSystem parameters for %s\n"), *(Object->GetPathName()));
				for (int32 EmitterIndex = 0; EmitterIndex < Object->Emitters.Num(); EmitterIndex++)
				{
					ClipboardString += FString::Printf(TEXT("\tEmitter %2d - "), EmitterIndex);
					if (UParticleEmitter* Emitter = Object->Emitters[EmitterIndex])
					{
						ClipboardString += FString::Printf(TEXT("%s\n"), *(Emitter->GetEmitterName().ToString()));
					}
					else
					{
						ClipboardString += FString(TEXT("* EMPTY *\n"));
					}

					TArray<FString>& ParticleSysParams = ParticleSysParamList[EmitterIndex];
					for (int32 PSPIndex = 0; PSPIndex < ParticleSysParams.Num(); PSPIndex++)
					{
						if (PSPIndex == 0)
						{
							ClipboardString += FString(TEXT("\t\tParticleSysParam List\n"));
						}
						ClipboardString += FString::Printf(TEXT("\t\t\t%s"), *(ParticleSysParams[PSPIndex]));
					}

					TArray<FString>& ParticleParameters = ParticleParameterList[EmitterIndex];
					for (int32 PPIndex = 0; PPIndex < ParticleParameters.Num(); PPIndex++)
					{
						if (PPIndex == 0)
						{
							ClipboardString += FString(TEXT("\t\tParticleParameter List\n"));
						}
						ClipboardString += FString::Printf(TEXT("\t\t\t%s"), *(ParticleParameters[PPIndex]));
					}
				}
			}
		}

		FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
	}

	void ConvertToSeeded(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		const TArray<UParticleSystem*> Objects = CBContext->LoadSelectedObjects<UParticleSystem>();

		ICascadeModule* CascadeModule = &FModuleManager::LoadModuleChecked<ICascadeModule>("Cascade");

		if(Objects.Num() > 0)
		{
			GWarn->BeginSlowTask( LOCTEXT("ParticleSystem_ConvertToSeeded_SlowTask", "Converting Particle Systems to Seeded"), true );
			for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
			{
				if (UParticleSystem* Object = *ObjIt)
				{
					GWarn->StatusUpdate( 
						ObjIt.GetIndex(), 
						Objects.Num(),
						FText::Format( LOCTEXT("ParticleSystem_ConvertToSeeded_StatusUpdate", "Converting {0} to Seeded"), FText::FromString( Object->GetName() ) ) );

					CascadeModule->ConvertModulesToSeeded(Object);
					CascadeModule->RefreshCascade(Object);
				}
			}
			GWarn->EndSlowTask();
		}
	}
		static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			
			// Particle System Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UParticleSystem::StaticClass());

				FToolMenuSection& ParticleSystemSection = Menu->FindOrAddSection("GetAssetActions");
				ParticleSystemSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == UParticleSystem::StaticClass()->GetClassPathName())
								{
									// Copy
									{
										const TAttribute<FText> Label = LOCTEXT("ParticleSystem_CopyParameters", "Copy Parameters");
										const TAttribute<FText> ToolTip = LOCTEXT("ParticleSystem_CopyParametersTooltip", "Copies particle system parameters to the clipboard.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCopyParameters);
										InSection.AddMenuEntry("ParticleSystem_CopyParameters", Label, ToolTip, Icon, UIAction);
									}

									// Convert
									{
										const TAttribute<FText> Label = LOCTEXT("ParticleSystem_ConvertToSeeded", "Convert To Seeded");
										const TAttribute<FText> ToolTip = LOCTEXT("ParticleSystem_ConvertToSeededToolTip", "Convert all modules in this particle system to random seeded modules.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ConvertToSeeded);
										InSection.AddMenuEntry("ParticleSystem_ConvertToSeeded", Label, ToolTip, Icon, UIAction);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
