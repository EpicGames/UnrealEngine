// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "Particles/ParticleSystem.h"
#include "AssetDefinition_ParticleSystem.generated.h"

UCLASS()
class UAssetDefinition_ParticleSystem : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ParticleSystem", "Cascade Particle System (Deprecated)"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(255,255,255); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UParticleSystem::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("AssetDefinition", "AssetDefinition_ParticleSystemSubMenu", "Deprecated"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
