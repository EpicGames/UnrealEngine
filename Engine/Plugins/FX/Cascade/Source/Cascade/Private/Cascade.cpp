// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cascade.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Components/VectorFieldComponent.h"
#include "Engine/InterpCurveEdSetup.h"
#include "CascadeConfiguration.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"
#include "CascadeParticleSystemComponent.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Preferences/CascadeOptions.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionFloatUniformCurve.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorUniformCurve.h"
#include "UObject/UObjectHash.h"
#include "Engine/Selection.h"
#include "CascadeModule.h"
#include "FXSystem.h"
#include "ObjectTools.h"
#include "DistCurveEditorModule.h"
#include "CascadePreviewViewportClient.h"
#include "SCascadeEmitterCanvas.h"
#include "CascadeEmitterCanvasClient.h"
#include "SCascadePreviewViewport.h"
#include "CascadeActions.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Colors/SColorPicker.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#include "Particles/Event/ParticleModuleEventGenerator.h"
#include "Particles/Parameter/ParticleModuleParameterDynamic.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/VectorField/ParticleModuleVectorFieldLocal.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "Particles/ParticleModuleRequired.h"

#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Framework/Commands/GenericCommands.h"
#include "UnrealEngine.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceScene.h"


static const FName Cascade_PreviewViewportTab("Cascade_PreviewViewport");
static const FName Cascade_EmmitterCanvasTab("Cascade_EmitterCanvas");
static const FName Cascade_PropertiesTab("Cascade_Properties");
static const FName Cascade_CurveEditorTab("Cascade_CurveEditor");

DEFINE_LOG_CATEGORY(LogCascade);

FRandomStream FCascade::RandomStream(FPlatformTime::Cycles());

FCascade::FCascade()
	: ParticleSystem(nullptr)
	, ParticleSystemComponent(nullptr)
	, LocalVectorFieldPreviewComponent(nullptr)
	, EditorOptions(nullptr)
	, EditorConfig(nullptr)
	, SelectedModule(nullptr)
	, SelectedEmitter(nullptr)
	, CopyModule(nullptr)
	, CopyEmitter(nullptr)
	, CurveToReplace(nullptr)
{
}

void FCascade::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(NSLOCTEXT("Cascade", "WorkspaceMenu_Cascade", "Cascade"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( Cascade_PreviewViewportTab, FOnSpawnTab::CreateSP( this, &FCascade::SpawnTab, Cascade_PreviewViewportTab ) )
		.SetDisplayName(NSLOCTEXT("Cascade", "SummonViewport", "Viewport"))
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner( Cascade_EmmitterCanvasTab, FOnSpawnTab::CreateSP( this, &FCascade::SpawnTab, Cascade_EmmitterCanvasTab ) )
		.SetDisplayName(NSLOCTEXT("Cascade", "SummonCanvas", "Emitters"))
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Emitter"));

	InTabManager->RegisterTabSpawner( Cascade_PropertiesTab, FOnSpawnTab::CreateSP( this, &FCascade::SpawnTab, Cascade_PropertiesTab ) )
		.SetDisplayName(NSLOCTEXT("Cascade", "SummonProperties", "Details"))
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner( Cascade_CurveEditorTab, FOnSpawnTab::CreateSP( this, &FCascade::SpawnTab, Cascade_CurveEditorTab ) )
		.SetDisplayName(NSLOCTEXT("Cascade", "SummonCurveEditor", "CurveEditor"))
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CurveBase"));
}

void FCascade::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( Cascade_PreviewViewportTab );
	InTabManager->UnregisterTabSpawner( Cascade_EmmitterCanvasTab );
	InTabManager->UnregisterTabSpawner( Cascade_PropertiesTab );
	InTabManager->UnregisterTabSpawner( Cascade_CurveEditorTab );
}

FCascade::~FCascade()
{
	UE_LOGF(LogCascade,Log,"Quitting Cascade. FXSystem=0x%p",GetFXSystem());

	GEditor->UnregisterForUndo(this);
	// If the user opened the geometry properties window, we request it be destroyed.
	TSharedPtr<SWindow> WindowPtr = GeometryPropertiesWindow.Pin();
	GeometryPropertiesWindow = NULL;

	if ( WindowPtr.IsValid() )
	{
		WindowPtr->RequestDestroyWindow();
	}

	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->ResetParticles(/*bEmptyInstances=*/ true);
		ParticleSystemComponent->CascadePreviewViewportPtr = nullptr;

		// Reset the detail mode values
		for (TObjectIterator<UParticleSystemComponent> It; It; ++It)
		{
			if (It->Template == ParticleSystemComponent->Template)
			{
				It->EditorDetailMode = -1;
			}
		}
	}

	if (ParticleSystem != NULL)
	{
		ParticleSystem->TurnOffSoloing();
	}

	DestroyColorPicker();

	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		// Save the preview scene
		PreviewViewport->GetViewportClient()->GetPreviewScene().SaveSettings(TEXT("CascadeEditor"));
		
		UStaticMeshComponent* FloorComponent = PreviewViewport->GetViewportClient()->GetFloorComponent();
		if (FloorComponent)
		{
			EditorOptions->FloorPosition = FloorComponent->GetRelativeLocation();
			EditorOptions->FloorRotation = FloorComponent->GetRelativeRotation();
			EditorOptions->FloorScale3D = FloorComponent->GetRelativeScale3D();

			if (FloorComponent->GetStaticMesh())
			{
				if (FloorComponent->GetStaticMesh()->GetOuter())
				{
					EditorOptions->FloorMesh = FloorComponent->GetStaticMesh()->GetOuter()->GetName();
					EditorOptions->FloorMesh += TEXT(".");
				}
				else
				{
					UE_LOGF(LogCascade, Warning, "Unable to locate Cascade floor mesh outer...");
					EditorOptions->FloorMesh = TEXT("");
				}

				EditorOptions->FloorMesh += FloorComponent->GetStaticMesh()->GetName();
			}
			else
			{
				EditorOptions->FloorMesh += FString::Printf(TEXT("/Engine/EditorMeshes/AnimTreeEd_PreviewFloor.AnimTreeEd_PreviewFloor"));
			}

			FString	Name;

			Name = EditorOptions->FloorMesh;

			EditorOptions->SaveConfig();
		}
	}

	ICascadeModule* CascadeModule = &FModuleManager::GetModuleChecked<ICascadeModule>("Cascade");
	CascadeModule->CascadeClosed(this);
}

void FCascade::OnComponentActivationChange(UParticleSystemComponent* PSC, bool bActivated)
{
	check(PSC);

	UCascadeParticleSystemComponent* CPSC = Cast<UCascadeParticleSystemComponent>(PSC);

	if (CPSC && CPSC->CascadePreviewViewportPtr)
	{
		if (FCascade* Cascade = CPSC->CascadePreviewViewportPtr->GetCascade())
		{
			PSC->SetManagingSignificance(true);
			PSC->SetRequiredSignificance(Cascade->GetRequiredSignificance());
		}
	}
}

void FCascade::InitCascade(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit)
{
	ParticleSystem = CastChecked<UParticleSystem>(ObjectToEdit);

	ParticleSystem->EditorLODSetting = 0;
	ParticleSystem->SetupLODValidity();

	// Support undo/redo
	ParticleSystem->SetFlags(RF_Transactional);

	CurrentLODIdx = 0;

	EditorOptions = NewObject<UCascadeOptions>();
	check(EditorOptions);
	EditorConfig = NewObject<UCascadeConfiguration>();
	check(EditorConfig);

	FString Description;
	for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIdx];
		if (Emitter)
		{
			Description += FString::Printf(TEXT("Emitter%d["), EmitterIdx);
			Emitter->SetFlags(RF_Transactional);
			for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* LODLevel = Emitter->GetLODLevel(LODIndex);
				if (LODLevel)
				{
					Description += FString::Printf(TEXT("LOD%d("), LODIndex);
					LODLevel->SetFlags(RF_Transactional);
					check(LODLevel->RequiredModule);
					LODLevel->RequiredModule->SetTransactionFlag();
					check(LODLevel->SpawnModule);
					LODLevel->SpawnModule->SetTransactionFlag();
					if (LODLevel->Modules.Num() > 0)
					{
						Description += FString::Printf(TEXT("Modules%d"), LODLevel->Modules.Num());
						for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							UParticleModule* pkModule = LODLevel->Modules[ModuleIdx];
							pkModule->SetTransactionFlag();
						}
					}
					Description += TEXT(")");
					if (Emitter->LODLevels.Num() > LODIndex + 1)
					{
						Description += TEXT(",");
					}
				}
			}
			Description += TEXT("]");
			if (ParticleSystem->Emitters.Num() > EmitterIdx + 1)
			{
				Description += TEXT(",");
			}
		}
	}
	if (Description.Len() > 0 && FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.Init"), TEXT("Overview"), Description);
	}

	ParticleSystemComponent = NewObject<UCascadeParticleSystemComponent>();

	LocalVectorFieldPreviewComponent = NewObject<UVectorFieldComponent>();

	bIsSoloing = false;

	bTransactionInProgress = false;

	SetSelectedModule(NULL, NULL);

	CopyModule = NULL;
	CopyEmitter = NULL;

	CurveToReplace = NULL;
	DetailMode = GlobalDetailMode = GetCachedScalabilityCVars().DetailMode;
	RequiredSignificance = EParticleSignificanceLevel::Low;

	bIsToggleMotion = false;
	MotionModeRadius = EditorOptions->MotionModeRadius;
	AccumulatedMotionTime = 0.0f;
	TimeScale = CachedTimeScale = 1.0f;
	bIsToggleLoopSystem = true;
	bIsPendingReset = false;
	ResetTime = BIG_NUMBER;
	TotalTime = 0.0;
	ParticleMemoryUpdateTime = 5.0f;

	bParticleModuleClassesInitialized = false;

	InitParticleModuleClasses();

	// Create a new emitter if the particle system is empty...
	if (ParticleSystem->Emitters.Num() == 0)
	{
		OnNewEmitter();
	}

	GEditor->RegisterForUndo(this);

	// Register our commands. This will only register them if not previously registered
	FCascadeCommands::Register();

	BindCommands();

	CreateInternalWidgets();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_Cascade_Layout_v3" )
	->AddArea(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->SetSizeCoefficient(0.9f)
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.3f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack() ->AddTab( Cascade_PreviewViewportTab, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack() ->AddTab( Cascade_PropertiesTab, ETabState::OpenedTab )
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.7f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.6f) ->AddTab( Cascade_EmmitterCanvasTab, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.4f) ->AddTab( Cascade_CurveEditorTab, ETabState::OpenedTab )
				)
			)	
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, CascadeAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);

	ICascadeModule* CascadeModule = &FModuleManager::LoadModuleChecked<ICascadeModule>( "Cascade" );
	AddMenuExtender(CascadeModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// @todo toolkit world centric editing
	/*if(IsWorldCentricAssetEditor())
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(FName(TEXT("Cascade_PreviewViewport")), FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(FName(TEXT("Cascade_EmitterCanvas")), FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(FName(TEXT("Cascade_Properties")), FString(), EToolkitTabSpot::Details);
		SpawnToolkitTab(FName(TEXT("Cascade_CurveEditor")), FString(), EToolkitTabSpot::Details);
	}*/
}

UParticleSystem* FCascade::GetParticleSystem() const
{
	return ParticleSystem;
}

UCascadeParticleSystemComponent* FCascade::GetParticleSystemComponent() const
{
	return ParticleSystemComponent;
}

UVectorFieldComponent* FCascade::GetLocalVectorFieldComponent() const
{
	return LocalVectorFieldPreviewComponent;
}

FFXSystemInterface* FCascade::GetFXSystem() const
{
	check(PreviewViewport.IsValid());
	auto World = PreviewViewport->GetViewportClient()->GetPreviewScene().GetWorld();
	check(World);
	return World->FXSystem.Get();
}

UParticleEmitter* FCascade::GetSelectedEmitter() const
{
	return SelectedEmitter;
}

UParticleModule* FCascade::GetSelectedModule() const
{
	return SelectedModule;
}

int32 FCascade::GetSelectedModuleIndex()
{
	return SelectedModuleIndex;
}

int32 FCascade::GetCurrentlySelectedLODLevelIndex() const
{
	int32 SetLODLevelIndex = CurrentLODIdx;
	if (SetLODLevelIndex < 0)
	{
		SetLODLevelIndex = 0;
	}
	else
	{
		if (ParticleSystem && (ParticleSystem->Emitters.Num() > 0))
		{
			UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
			if (Emitter)
			{
				if (SetLODLevelIndex >= Emitter->LODLevels.Num())
				{
					SetLODLevelIndex = Emitter->LODLevels.Num() - 1;
				}
			}
		}
		else
		{
			SetLODLevelIndex = 0;
		}
	}

	return SetLODLevelIndex;
}

UParticleLODLevel* FCascade::GetCurrentlySelectedLODLevel() const
{
	int32 CurrentLODLevel = GetCurrentlySelectedLODLevelIndex();
	if ((CurrentLODLevel >= 0) && SelectedEmitter)
	{
		return SelectedEmitter->GetLODLevel(CurrentLODLevel);
	}

	return NULL;
}

UParticleLODLevel* FCascade::GetCurrentlySelectedLODLevel(UParticleEmitter* InEmitter)
{
	if (InEmitter)
	{
		UParticleEmitter* SaveSelectedEmitter = SelectedEmitter;
		SelectedEmitter = InEmitter;
		UParticleLODLevel* ReturnLODLevel = GetCurrentlySelectedLODLevel();
		SelectedEmitter = SaveSelectedEmitter;
		return ReturnLODLevel;
	}

	return NULL;
}

UCascadeOptions* FCascade::GetEditorOptions() const
{
	return EditorOptions;
}

UCascadeConfiguration* FCascade::GetEditorConfiguration() const
{
	return EditorConfig;
}

TSharedPtr<IDistributionCurveEditor> FCascade::GetCurveEditor()
{
	return CurveEditor;
}

TSharedPtr<SCascadePreviewViewport> FCascade::GetPreviewViewport()
{
	return PreviewViewport;
}

bool FCascade::GetIsSoloing() const
{
	return bIsSoloing;
}

void FCascade::SetIsSoloing(bool bInIsSoloing)
{
	bIsSoloing = bInIsSoloing;
}

int32 FCascade::GetDetailMode() const
{
	return DetailMode;
}

EParticleSignificanceLevel FCascade::GetRequiredSignificance() const
{
	return RequiredSignificance;
}

bool FCascade::GetIsModuleShared(UParticleModule* Module)
{
	int32 FindCount = 0;

	UParticleModuleSpawn* SpawnModule = Cast<UParticleModuleSpawn>(Module);
	UParticleModuleRequired* RequiredModule = Cast<UParticleModuleRequired>(Module);
	UParticleModuleTypeDataBase* TypeDataModule = Cast<UParticleModuleTypeDataBase>(Module);

	int32	CurrLODSetting	= GetCurrentlySelectedLODLevelIndex();
	if (CurrLODSetting < 0)
	{
		return false;
	}

	for (int32 i = 0; i < ParticleSystem->Emitters.Num(); i++)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[i];
		UParticleLODLevel* LODLevel = Emitter->GetLODLevel(CurrLODSetting);
		if (LODLevel == NULL)
		{
			continue;
		}

		if (SpawnModule)
		{
			if (SpawnModule == LODLevel->SpawnModule)
			{
				FindCount++;
				if (FindCount >= 2)
				{
					return true;
				}
			}
		}
		else if (RequiredModule)
		{
			if (RequiredModule == LODLevel->RequiredModule)
			{
				FindCount++;
				if (FindCount >= 2)
				{
					return true;
				}
			}
		}
		else if (TypeDataModule)
		{
			if (TypeDataModule == LODLevel->TypeDataModule)
			{
				FindCount++;
				if (FindCount >= 2)
				{
					return true;
				}
			}
		}
		else
		{
			for (int32 j = 0; j < LODLevel->Modules.Num(); j++)
			{
				if (LODLevel->Modules[j] == Module)
				{
					FindCount++;
					if (FindCount == 2)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void FCascade::ToggleEnableOnSelectedEmitter(UParticleEmitter* InEmitter)
{
	if (!InEmitter)
	{
		return;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "ToggleEnableOnSelectedEmitter", "Toggle Enable on Emitter");

	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( Transaction ) == false)
		{
			return;
		}

		// Make them toggle again in this case as the setting may/maynot be what they think it is...
		ParticleSystem->SetupSoloing();
		return;
	}

	UParticleLODLevel* LODLevel = GetCurrentlySelectedLODLevel(InEmitter);
	if (LODLevel)
	{
		BeginTransaction( Transaction );

		ModifyParticleSystem();
		ParticleSystem->PreEditChange(NULL);

		LODLevel->bEnabled = !LODLevel->bEnabled;
		LODLevel->RequiredModule->bEnabled = LODLevel->bEnabled;

		ParticleSystem->PostEditChange();
		ParticleSystem->SetupSoloing();

		OnRestartInLevel();

		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}

		EndTransaction( Transaction );
		ParticleSystem->MarkPackageDirty();
	}
}

bool FCascade::AddSelectedToGraph(TArray<const FCurveEdEntry*>& OutCurveEntries)
{
	bool bNewCurve = false;
	if (SelectedEmitter)
	{
		int32 CurrLODSetting = GetCurrentlySelectedLODLevelIndex();
		if (SelectedEmitter->IsLODLevelValid(CurrLODSetting) )
		{
			if (SelectedModule)
			{
				UParticleLODLevel* LODLevel = SelectedEmitter->GetLODLevel(CurrLODSetting);
				if (LODLevel->IsModuleEditable(SelectedModule))
				{
					bNewCurve = SelectedModule->AddModuleCurvesToEditor(ParticleSystem->CurveEdSetup, OutCurveEntries);
					CurveEditor->CurveChanged();
				}
			}

			SetSelectedInCurveEditor();
			CurveEditor->RefreshViewport();
		}
	}
	return bNewCurve;
}

void FCascade::ShowDesiredCurvesOnly(const TArray<const FCurveEdEntry*>& InCurveEntries)
{
	CurveEditor->ClearAllVisibleCurves();
	for( int32 CurveIndex = 0; CurveIndex < InCurveEntries.Num(); CurveIndex++ )
	{
		const FCurveEdEntry* CurveEntry = InCurveEntries[CurveIndex];
		check( CurveEntry );
		CurveEditor->SetCurveVisible( CurveEntry->CurveObject, true );
	}
	CurveEditor->CurveChanged();
}

void FCascade::SetSelectedEmitter(UParticleEmitter* NewSelectedEmitter, bool bIsSimpleAssignment/* = false*/)
{
	if (!bIsSimpleAssignment)
	{
		SetSelectedModule(NewSelectedEmitter, NULL);
	}
	else
	{
		SelectedEmitter = NewSelectedEmitter;
	}
}

void FCascade::SetSelectedModule(UParticleModule* NewSelectedModule)
{
	SelectedModule = NewSelectedModule;
}

void FCascade::SetSelectedModule(UParticleEmitter* NewSelectedEmitter, UParticleModule* NewSelectedModule)
{
	bool bIsNewEmitter = SelectedEmitter != NewSelectedEmitter;
	bool bIsNewModule = SelectedModule != NewSelectedModule;

	SelectedEmitter = NewSelectedEmitter;

	int32 CurrLODIndex = GetCurrentlySelectedLODLevelIndex();
	if (CurrLODIndex < 0)
	{
		return;
	}

	UParticleLODLevel* LODLevel = NULL;
	// Make sure it's the correct LOD level...
	if (SelectedEmitter)
	{
		LODLevel = SelectedEmitter->GetLODLevel(CurrLODIndex);
		if (NewSelectedModule)
		{
			int32	ModuleIndex	= INDEX_NONE;
			for (int32 LODLevelCheck = 0; LODLevelCheck < SelectedEmitter->LODLevels.Num(); LODLevelCheck++)
			{
				UParticleLODLevel* CheckLODLevel = SelectedEmitter->LODLevels[LODLevelCheck];
				if (LODLevel)
				{
					// Check the type data...
					if (CheckLODLevel->TypeDataModule &&
						(CheckLODLevel->TypeDataModule == NewSelectedModule))
					{
						ModuleIndex = INDEX_TYPEDATAMODULE;
					}

					// Check the required module...
					if (ModuleIndex == INDEX_NONE)
					{
						if (CheckLODLevel->RequiredModule == NewSelectedModule)
						{
							ModuleIndex = INDEX_REQUIREDMODULE;
						}
					}

					// Check the spawn...
					if (ModuleIndex == INDEX_NONE)
					{
						if (CheckLODLevel->SpawnModule == NewSelectedModule)
						{
							ModuleIndex = INDEX_SPAWNMODULE;
						}
					}

					// Check the rest...
					if (ModuleIndex == INDEX_NONE)
					{
						for (int32 ModuleCheck = 0; ModuleCheck < CheckLODLevel->Modules.Num(); ModuleCheck++)
						{
							if (CheckLODLevel->Modules[ModuleCheck] == NewSelectedModule)
							{
								ModuleIndex = ModuleCheck;
								break;
							}
						}
					}
				}

				if (ModuleIndex != INDEX_NONE)
				{
					break;
				}
			}

			switch (ModuleIndex)
			{
			case INDEX_NONE:
				break;
			case INDEX_TYPEDATAMODULE:
				NewSelectedModule = LODLevel->TypeDataModule;
				break;
			case INDEX_REQUIREDMODULE:
				NewSelectedModule = LODLevel->RequiredModule;
				break;
			case INDEX_SPAWNMODULE:
				NewSelectedModule = LODLevel->SpawnModule;
				break;
			default:
				NewSelectedModule = LODLevel->Modules[ModuleIndex];
				break;
			}
			SelectedModuleIndex	= ModuleIndex;
		}
	}

	SelectedModule = NewSelectedModule;

	TArray<UObject*> NewSelection;
	bool bReadOnly = false;
	UObject* PropObj = ParticleSystem;
	if (SelectedEmitter)
	{
		if (SelectedModule)
		{
			if (LODLevel != NULL)
			{
				if (bReadOnly == false) //-V547
				{
					if (LODLevel->Level != CurrLODIndex)
					{
						bReadOnly = true;
					}
					else
					{
						bReadOnly = !LODLevel->IsModuleEditable(SelectedModule);
					}
				}
			}
			PropObj = SelectedModule;
		}
		else
		{
			if (bReadOnly == false) //-V547
			{
				// Only allowing editing the SelectedEmitter 
				// properties when at the highest LOD level.
				if (!(LODLevel && (LODLevel->Level == 0)))
				{
					bReadOnly = true;
				}
			}
			PropObj = SelectedEmitter;
		}

		// If soloing and NOT an emitter that is soloing, mark it Read-only!
		if (bIsSoloing == true)
		{
			if (SelectedEmitter->bIsSoloing == false)
			{
				bReadOnly = true;
			}
		}
	}
	
	NewSelection.Add(PropObj);
	SetSelection(NewSelection);

	if (Details.IsValid())
	{
		Details->SetEnabled(!bReadOnly);
	}

	SetSelectedInCurveEditor();

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::SetSelection(TArray<UObject*> SelectedObjects)
{
	if (Details.IsValid())
	{
		for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
		{
			if (!SelectedObjects[Idx])
			{
				return;
			}
		}

		Details->SetObjects(SelectedObjects);
	}
}

TArray<UClass*>& FCascade::GetParticleModuleBaseClasses()
{
	return ParticleModuleBaseClasses;
}

TArray<UClass*>& FCascade::GetParticleModuleClasses()
{
	return ParticleModuleClasses;
}

void FCascade::OnCustomModuleOption(int32 Idx)
{
	if (SelectedModule != NULL)
	{
		// Run it on the selected module
		if (SelectedModule->PerformCustomMenuEntry(Idx) == true)
		{
			UParticleModule* SaveModule = SelectedModule;
			SetSelectedModule(SelectedEmitter, NULL);
			SetSelectedModule(SelectedEmitter, SaveModule);
		}
	}
}

void FCascade::OnNewModule(int32 Idx)
{
	if (SelectedEmitter == NULL)
	{
		return;
	}

	int32 CurrLODLevel = GetCurrentlySelectedLODLevelIndex();
	if (CurrLODLevel != 0)
	{
		// Don't allow creating modules if not at highest LOD
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Cascade", "CascadeLODAddError", "Must be on lowest LOD (0) to create modules"));
		return;
	}

	UClass* NewModClass = ParticleModuleClasses[Idx];
	check(NewModClass->IsChildOf(UParticleModule::StaticClass()));

	bool bIsEventGenerator = false;

	if (NewModClass->IsChildOf(UParticleModuleTypeDataBase::StaticClass()))
	{
		// Make sure there isn't already a TypeData module applied!
		UParticleLODLevel* LODLevel = SelectedEmitter->GetLODLevel(0);
		if (LODLevel->TypeDataModule != 0)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_TypeDataModuleAlreadyPresent", "A TypeData module is already present.\nPlease remove it first."));
			return;
		}
	}
	else if (NewModClass == UParticleModuleEventGenerator::StaticClass())
	{
		bIsEventGenerator = true;
		// Make sure there isn't already an EventGenerator module applied!
		UParticleLODLevel* LODLevel = SelectedEmitter->GetLODLevel(0);
		if (LODLevel->EventGenerator != NULL)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_EventGeneratorModuleAlreadyPresent", "An EventGenerator module is already present.\nPlease remove it first."));
			return;
		}
	}
	else if (NewModClass == UParticleModuleParameterDynamic::StaticClass())
	{
		// Make sure there isn't already an DynamicParameter module applied!
		UParticleLODLevel* LODLevel = SelectedEmitter->GetLODLevel(0);
		for (int32 CheckMod = 0; CheckMod < LODLevel->Modules.Num(); CheckMod++)
		{
			UParticleModuleParameterDynamic* DynamicParamMod = Cast<UParticleModuleParameterDynamic>(LODLevel->Modules[CheckMod]);
			if (DynamicParamMod)
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_DynamicParameterModuleAlreadyPresent", "A DynamicParameter module is already present.\nPlease remove it first."));
				return;
			}
		}
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "CreateNewModule", "Create New Module");

	BeginTransaction( Transaction );
	ModifyParticleSystem();
	ModifySelectedObjects();

	if( NewModClass == UParticleModuleTypeDataMesh::StaticClass() )
	{
		// TypeDataMeshes update the LOD levels RequiredModule material so mark it for transaction
		UParticleLODLevel* LODLevel = SelectedEmitter->GetLODLevel( 0 );
		if( LODLevel->RequiredModule )
		{
			LODLevel->RequiredModule->Modify();
		}
	}

	ParticleSystem->PreEditChange(NULL);
	ParticleSystemComponent->PreEditChange(NULL);

	// Construct it and add to selected emitter.
	UParticleModule* NewModule = NewObject<UParticleModule>(ParticleSystem, NewModClass, NAME_None, RF_Transactional);
	NewModule->ModuleEditorColor = FColor::MakeRandomColor();
	NewModule->SetToSensibleDefaults(SelectedEmitter);
	NewModule->LODValidity = 1;
	NewModule->SetTransactionFlag();

	UParticleLODLevel* LODLevel	= SelectedEmitter->GetLODLevel(0);
	if (bIsEventGenerator == true)
	{
		LODLevel->Modules.Insert(NewModule, 0);
		LODLevel->EventGenerator = Cast<UParticleModuleEventGenerator>(NewModule);
	}
	else
	{
		LODLevel->Modules.Add(NewModule);
	}

	for (int32 LODIndex = 1; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
	{
		LODLevel = SelectedEmitter->GetLODLevel(LODIndex);
		NewModule->LODValidity |= (1 << LODIndex);
		if (bIsEventGenerator == true)
		{
			LODLevel->Modules.Insert(NewModule, 0);
			LODLevel->EventGenerator = Cast<UParticleModuleEventGenerator>(NewModule);
		}
		else
		{
			LODLevel->Modules.Add(NewModule);
		}
	}

	SelectedEmitter->UpdateModuleLists();

	ParticleSystemComponent->PostEditChange();
	ParticleSystem->PostEditChange();

	EndTransaction( Transaction );

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.NewModule"), TEXT("Class"), NewModClass->GetName());
	}

	ParticleSystem->MarkPackageDirty();

	// Refresh viewport
	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnNewEmitter()
{
	FText Transaction = NSLOCTEXT("UnrealEd", "NewEmitter", "Create New Emitter");

	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( Transaction ) == false)
		{
			return;
		}
	}

	BeginTransaction( Transaction );
	ParticleSystem->PreEditChange(NULL);
	ParticleSystemComponent->PreEditChange(NULL);

	UClass* NewEmitClass = UParticleSpriteEmitter::StaticClass();

	// Construct it
	UParticleEmitter* NewEmitter = NewObject<UParticleEmitter>(ParticleSystem, NewEmitClass, NAME_None, RF_Transactional);
	UParticleLODLevel* LODLevel	= NewEmitter->GetLODLevel(0);
	if (LODLevel == NULL)
	{
		// Generate the HighLOD level, and the default lowest level
		int32 Index = NewEmitter->CreateLODLevel(0);
		LODLevel = NewEmitter->GetLODLevel(0);
	}

	check(LODLevel);

	NewEmitter->EmitterEditorColor = FColor::MakeRandomColor();
	NewEmitter->EmitterEditorColor.A = 255;

	// Set to sensible default values
	NewEmitter->SetToSensibleDefaults();

	// Handle special cases...
	if (NewEmitClass == UParticleSpriteEmitter::StaticClass())
	{
		// For handyness- use currently selected material for new emitter (or default if none selected)
		UParticleSpriteEmitter* NewSpriteEmitter = (UParticleSpriteEmitter*)NewEmitter;
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
		UMaterialInterface* CurrentMaterial = GEditor->GetSelectedObjects()->GetTop<UMaterialInterface>();
		if (CurrentMaterial)
		{
			LODLevel->RequiredModule->Material = CurrentMaterial;
		}
		else
		{
			LODLevel->RequiredModule->Material = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EngineMaterials/DefaultParticle.DefaultParticle"), NULL, LOAD_None, NULL);
		}
	}

	// Generate all the levels that are present in other emitters...
	if (ParticleSystem->Emitters.Num() > 0)
	{
		UParticleEmitter* ExistingEmitter = ParticleSystem->Emitters[0];

		if (ExistingEmitter->LODLevels.Num() > 1)
		{
			if (NewEmitter->AutogenerateLowestLODLevel(ParticleSystem->bRegenerateLODDuplicate) == false)
			{
				UE_LOGF(LogCascade, Warning, "Failed to autogenerate lowest LOD level!");
			}
		}

		if (ExistingEmitter->LODLevels.Num() > 2)
		{
			UE_LOGF(LogCascade, Log, "Generating existing LOD levels...");

			// Walk the LOD levels of the existing emitter...
			UParticleLODLevel* ExistingLOD;
			UParticleLODLevel* NewLOD_Prev = NewEmitter->LODLevels[0];
			UParticleLODLevel* NewLOD_Next = NewEmitter->LODLevels[1];

			check(NewLOD_Prev);
			check(NewLOD_Next);

			for (int32 LODIndex = 1; LODIndex < ExistingEmitter->LODLevels.Num() - 1; LODIndex++)
			{
				ExistingLOD = ExistingEmitter->LODLevels[LODIndex];

				// Add this one
				int32 ResultIndex = NewEmitter->CreateLODLevel(ExistingLOD->Level, true);

				UParticleLODLevel* NewLODLevel	= NewEmitter->LODLevels[ResultIndex];
				check(NewLODLevel);
				NewLODLevel->UpdateModuleLists();
			}
		}
	}

	NewEmitter->UpdateModuleLists();

	NewEmitter->PostEditChange();

	NewEmitter->SetFlags(RF_Transactional);
	for (int32 LODIndex = 0; LODIndex < NewEmitter->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* NewEmitterLODLevel = NewEmitter->GetLODLevel(LODIndex);
		if (NewEmitterLODLevel)
		{
			NewEmitterLODLevel->SetFlags(RF_Transactional);
			check(NewEmitterLODLevel->RequiredModule);
			NewEmitterLODLevel->RequiredModule->SetTransactionFlag();
			check(NewEmitterLODLevel->SpawnModule);
			NewEmitterLODLevel->SpawnModule->SetTransactionFlag();
			for (int32 jj = 0; jj < NewEmitterLODLevel->Modules.Num(); jj++)
			{
				UParticleModule* pkModule = NewEmitterLODLevel->Modules[jj];
				pkModule->SetTransactionFlag();
			}
		}
	}

	// Add to selected emitter
	ParticleSystem->Emitters.Add(NewEmitter);

	// Setup the LOD distances
	if (ParticleSystem->LODDistances.Num() == 0)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
		if (Emitter)
		{
			ParticleSystem->LODDistances.AddUninitialized(Emitter->LODLevels.Num());
			for (int32 LODIndex = 0; LODIndex < ParticleSystem->LODDistances.Num(); LODIndex++)
			{
				ParticleSystem->LODDistances[LODIndex] = LODIndex * 2500.0f;
			}
		}
	}
	if (ParticleSystem->LODSettings.Num() == 0)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
		if (Emitter)
		{
			ParticleSystem->LODSettings.AddUninitialized(Emitter->LODLevels.Num());
			for (int32 LODIndex = 0; LODIndex < ParticleSystem->LODSettings.Num(); LODIndex++)
			{
				ParticleSystem->LODSettings[LODIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
			}
		}
	}

	ParticleSystemComponent->PostEditChange();
	ParticleSystem->PostEditChange();

	ParticleSystem->SetupSoloing();

	EndTransaction( Transaction );

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.NewEmitter"));
	}

	// Refresh viewport
	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::SetCopyEmitter(UParticleEmitter* NewEmitter)
{
	CopyEmitter = NewEmitter;
}

void FCascade::SetCopyModule(UParticleEmitter* NewEmitter, UParticleModule* NewModule)
{
	CopyEmitter = NewEmitter;
	CopyModule = NewModule;
}

TArray<UParticleModule*>& FCascade::GetDraggedModuleList()
{
	return DraggedModuleList;
}

bool FCascade::InsertModule(UParticleModule* Module, UParticleEmitter* TargetEmitter, int32 TargetIndex, bool bSetSelected)
{
	if (!Module || !TargetEmitter || TargetIndex == INDEX_NONE)
	{
		return false;
	}

	int32 CurrLODIndex = GetCurrentlySelectedLODLevelIndex();
	if (CurrLODIndex != 0)
	{
		// Don't allow moving modules if not at highest LOD
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Cascade", "CascadeLODMoveError", "Must be on lowest LOD (0) to move modules"));
		return false;
	}

	// Cannot insert the same module more than once into the same emitter.
	UParticleLODLevel* LODLevel	= TargetEmitter->GetLODLevel(0);
	for(int32 i = 0; i < LODLevel->Modules.Num(); i++)
	{
		if (LODLevel->Modules[i] == Module)
		{
			FNotificationInfo Info( NSLOCTEXT("UnrealEd", "Error_ModuleCanOnlyBeUsedInEmitterOnce", "A Module can only be used in each Emitter once.") );
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return false;
		}
	}

	if (Module->IsA(UParticleModuleParameterDynamic::StaticClass()))
	{
		// Make sure there isn't already an DynamicParameter module applied!
		for (int32 CheckMod = 0; CheckMod < LODLevel->Modules.Num(); CheckMod++)
		{
			UParticleModuleParameterDynamic* DynamicParamMod = Cast<UParticleModuleParameterDynamic>(LODLevel->Modules[CheckMod]);
			if (DynamicParamMod)
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_DynamicParameterModuleAlreadyPresent", "A DynamicParameter module is already present.\nPlease remove it first."));
				return false;
			}
		}
	}

	// If the Spawn or Required modules are being 're-inserted', do nothing!
	if ((LODLevel->SpawnModule == Module) ||
		(LODLevel->RequiredModule == Module))
	{
		return false;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "InsertModule", "Insert Module");

	BeginTransaction( Transaction );
	ModifyEmitter(TargetEmitter);
	ModifyParticleSystem();

	// Insert in desired location in new Emitter
	ParticleSystem->PreEditChange(NULL);

	TArray<UParticleModule*>& DraggedModules = EmitterCanvas->GetViewportClient()->GetDraggedModules();

	if (Module->IsA(UParticleModuleTypeDataBase::StaticClass()))
	{
		bool bInsert = true;
		if (LODLevel->TypeDataModule != NULL)
		{
			// Prompt to ensure they want to replace the TDModule
			bInsert = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "Cascade_ReplaceTypeDataModule", "Replace the existing type data module?"));
		}

		if (bInsert == true)
		{
			LODLevel->TypeDataModule = CastChecked<UParticleModuleTypeDataBase>(Module);

			if (DraggedModules.Num() > 0)
			{
				// Swap the modules in all the LOD levels
				for (int32 LODIndex = 1; LODIndex < TargetEmitter->LODLevels.Num(); LODIndex++)
				{
					UParticleLODLevel* TargetEmitterLODLevel = TargetEmitter->GetLODLevel(LODIndex);
					UParticleModule* DraggedModule = DraggedModules[LODIndex];

					TargetEmitterLODLevel->TypeDataModule = CastChecked<UParticleModuleTypeDataBase>(DraggedModule);
				}
			}
		}
	}
	else if (Module->IsA(UParticleModuleSpawn::StaticClass()))
	{
		// There can be only one...
		LODLevel->SpawnModule = CastChecked<UParticleModuleSpawn>(Module);
		if (DraggedModules.Num() > 0)
		{
			// Swap the modules in all the LOD levels
			for (int32 LODIndex = 1; LODIndex < TargetEmitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* TargetEmitterLODLevel = TargetEmitter->GetLODLevel(LODIndex);
				UParticleModuleSpawn* DraggedModule = CastChecked<UParticleModuleSpawn>(DraggedModules[LODIndex]);
				TargetEmitterLODLevel->SpawnModule = DraggedModule;
			}
		}
	}
	else if (Module->IsA(UParticleModuleRequired::StaticClass()))
	{
		// There can be only one...
		LODLevel->RequiredModule = CastChecked<UParticleModuleRequired>(Module);
		if (DraggedModules.Num() > 0)
		{
			// Swap the modules in all the LOD levels
			for (int32 LODIndex = 1; LODIndex < TargetEmitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* TargetEmitterLODLevel = TargetEmitter->GetLODLevel(LODIndex);
				UParticleModuleRequired* DraggedModule = CastChecked<UParticleModuleRequired>(DraggedModules[LODIndex]);
				TargetEmitterLODLevel->RequiredModule = DraggedModule;
			}
		}
	}
	else
	{
		int32 NewModuleIndex = FMath::Clamp<int32>(TargetIndex, 0, LODLevel->Modules.Num());
		LODLevel->Modules.InsertUninitialized(NewModuleIndex);
		LODLevel->Modules[NewModuleIndex] = Module;

		if (DraggedModules.Num() > 0)
		{
			// Swap the modules in all the LOD levels
			for (int32 LODIndex = 1; LODIndex < TargetEmitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* TargetEmitterLODLevel = TargetEmitter->GetLODLevel(LODIndex);
				UParticleModule* DraggedModule = DraggedModules[LODIndex];

				TargetEmitterLODLevel->Modules.InsertUninitialized(NewModuleIndex);
				TargetEmitterLODLevel->Modules[NewModuleIndex] = DraggedModule;
			}
		}
	}

	DraggedModules.Empty();

	TargetEmitter->UpdateModuleLists();

	ParticleSystem->PostEditChange();

	// Update selection
	if (bSetSelected)
	{
		SetSelectedModule(TargetEmitter, Module);
	}

	EndTransaction( Transaction );

	ParticleSystem->MarkPackageDirty();
	
	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}

	return true;
}

void FCascade::CopyModuleToEmitter(UParticleModule* pkSourceModule, UParticleEmitter* pkTargetEmitter, UParticleSystem* pkTargetSystem, int32 TargetIndex)
{
	check(pkSourceModule);
	check(pkTargetEmitter);
	check(pkTargetSystem);

	int32 CurrLODIndex = GetCurrentlySelectedLODLevelIndex();
	if (CurrLODIndex != 0)
	{
		// Don't allow copying modules if not at highest LOD
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Cascade", "CascadeLODCopyError", "Must be on lowest LOD (0) to copy modules"));
		return;
	}

	UObject* DupObject = StaticDuplicateObject(pkSourceModule, pkTargetSystem);
	if (DupObject)
	{
		UParticleModule* Module	= Cast<UParticleModule>(DupObject);
		Module->ModuleEditorColor = FColor::MakeRandomColor();

		UParticleLODLevel* LODLevel;
		UParticleModule* DraggedModule = EmitterCanvas->GetViewportClient()->GetDraggedModule();
		TArray<UParticleModule*>& DraggedModules = EmitterCanvas->GetViewportClient()->GetDraggedModules();

		if (DraggedModule == pkSourceModule)
		{
			DraggedModules[0] = Module;
			// If we are dragging, we need to copy all the LOD modules
			for (int32 LODIndex = 1; LODIndex < pkTargetEmitter->LODLevels.Num(); LODIndex++)
			{
				LODLevel	= pkTargetEmitter->GetLODLevel(LODIndex);

				UParticleModule* CopySource = DraggedModules[LODIndex];
				if (CopySource)
				{
					DupObject = StaticDuplicateObject(CopySource, pkTargetSystem);
					if (DupObject)
					{
						UParticleModule* NewModule	= Cast<UParticleModule>(DupObject);
						NewModule->ModuleEditorColor = Module->ModuleEditorColor;
						DraggedModules[LODIndex] = NewModule;
					}
				}
				else
				{
					UE_LOGF(LogCascade, Warning, "Missing dragged module!");
				}
			}
		}

		LODLevel	= pkTargetEmitter->GetLODLevel(0);
		InsertModule(Module, pkTargetEmitter, (TargetIndex != INDEX_NONE) ? TargetIndex : LODLevel->Modules.Num(), false);
	}
}

FName FCascade::GetToolkitFName() const
{
	return FName("Cascade");
}

FText FCascade::GetBaseToolkitName() const
{
	return NSLOCTEXT("Cascade", "AppLabel", "Cascade" );
}

FString FCascade::GetWorldCentricTabPrefix() const
{
	return NSLOCTEXT("Cascade", "WorldCentricTabPrefix", "Cascade ").ToString();
}

FLinearColor FCascade::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

TSharedRef<SDockTab> FCascade::SpawnTab(const FSpawnTabArgs& SpawnTabArgs, FName TabIdentifier)
{
	if (TabIdentifier == FName(TEXT("Cascade_PreviewViewport")))
	{
		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
			.Label( NSLOCTEXT("Cascade", "CascadeViewportTitle", "Viewport"))
			[
				PreviewViewport.ToSharedRef()
			];

		PreviewViewport->ParentTab = SpawnedTab;

		// Set emitter to use the particle system we are editing.
		ParticleSystemComponent->SetTemplate(ParticleSystem);

		ParticleSystemComponent->InitializeSystem();
		ParticleSystemComponent->ActivateSystem();

		// Set camera position/orientation
		PreviewViewport->GetViewportClient()->SetPreviewCamera(ParticleSystem->ThumbnailAngle, ParticleSystem->ThumbnailDistance);
		PreviewViewport->GetViewportClient()->SetViewLocationForOrbiting(FVector::ZeroVector);

		return SpawnedTab;
	}
	else if (TabIdentifier == FName(TEXT("Cascade_EmitterCanvas")))
	{
		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
			.Label(NSLOCTEXT("Cascade", "CascadeGraphCanvasTitle", "Emitters"))
			[
				EmitterCanvas.ToSharedRef()
			];

		EmitterCanvas->ParentTab = SpawnedTab;

		return SpawnedTab;
	}
	else if(TabIdentifier == FName(TEXT("Cascade_Properties")))
	{
		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
			.Label(NSLOCTEXT("Cascade", "CascadePropertiesTitle", "Details"))
			[
				Details.ToSharedRef()
			];

		return SpawnedTab;
	}
	else if(TabIdentifier == FName(TEXT("Cascade_CurveEditor")))
	{
		return SNew(SDockTab)
			.Label(NSLOCTEXT("Cascade", "CascadeCurveEditorTitle", "Curve Editor"))
			[
				CurveEditor.ToSharedRef()
			];
	}
	else
	{
		ensure(false);
		return SNew(SDockTab);
	}
}

void FCascade::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->GetPreviewScene().AddReferencedObjects(Collector);
	}

	Collector.AddReferencedObject(ParticleSystem);
	Collector.AddReferencedObject(ParticleSystemComponent);
	Collector.AddReferencedObject(LocalVectorFieldPreviewComponent);
	Collector.AddReferencedObject(EditorOptions);
	Collector.AddReferencedObject(EditorConfig);
	Collector.AddReferencedObject(SelectedModule);
	Collector.AddReferencedObject(SelectedEmitter);
	Collector.AddReferencedObject(CopyModule);
	Collector.AddReferencedObject(CopyEmitter);
	Collector.AddReferencedObject(CurveToReplace);

	for (auto& Pair : EmitterToThumbnailMap)
	{
		Collector.AddReferencedObject(Pair.Value.Texture);
	}
}

FString FCascade::GetReferencerName() const
{
	return TEXT("FCascade");
}

void FCascade::Tick(float DeltaTime)
{
	// This is a bit of a hack. In order to not tick all open Cascade editors (which tick through engine tick) even when not visible,
	// the preview viewport keeps track of whether it has been ticked in the last frame. Slate widgets aren't ticked if invisible, so 
	// this will tell us if we should run simulation in this instance. If it hasn't ticked, we skip ticking Cascade as well and clear
	// the flag for the next frame
	if ( !PreviewViewport->HasJustTicked() )
	{
		return;
	}

	PreviewViewport->ClearTickFlag();



	static const double ResetInterval = 0.5f;

	// Clamp delta time.
	DeltaTime = FMath::Min<float>(DeltaTime, 0.040f);

	int32 DetailModeCVar = GetCachedScalabilityCVars().DetailMode;
	if (GlobalDetailMode != DetailModeCVar)
	{
		GlobalDetailMode = DetailModeCVar;
		OnDetailMode((EDetailMode)GlobalDetailMode);
	}

	static float LastMemUpdateTime = 0.0f;
	bool bCurrentlySoloing = false;
	if (ParticleSystem)
	{
		for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIdx];
			if (Emitter && (Emitter->bIsSoloing == true))
			{
				bCurrentlySoloing = true;
				break;
			}
		}

		LastMemUpdateTime += DeltaTime;
		if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid() &&
			LastMemUpdateTime > ParticleMemoryUpdateTime)
		{
			PreviewViewport->GetViewportClient()->UpdateMemoryInformation();
			LastMemUpdateTime = 0.0f;
		}
	}

	// Don't bother ticking at all if paused.
	if (TimeScale > KINDA_SMALL_NUMBER)
	{
		check(ParticleSystem != NULL);
		const float fSaveUpdateDelta = ParticleSystem->UpdateTime_Delta;
		if (TimeScale < 1.0f)
		{
			ParticleSystem->UpdateTime_Delta *= TimeScale;
		}

		float CurrDeltaTime = TimeScale * DeltaTime;

		if (bIsToggleMotion == true)
		{
			AccumulatedMotionTime += CurrDeltaTime;
			FVector Position;
			Position.X = MotionModeRadius * FMath::Sin(AccumulatedMotionTime);
			Position.Y = MotionModeRadius * FMath::Cos(AccumulatedMotionTime);
			Position.Z = 0.0f;
			ParticleSystemComponent->SetComponentToWorld(FTransform(Position));
		}

		if (ParticleSystemComponent->IsComponentTickEnabled())
		{
			ParticleSystemComponent->CascadeTickComponent(CurrDeltaTime, LEVELTICK_All);
		}
		ParticleSystemComponent->DoDeferredRenderUpdates_Concurrent();
		UWorld* World = PreviewViewport->GetViewportClient()->GetPreviewScene().GetWorld();
		GetFXSystem()->Tick(World, CurrDeltaTime);
		TotalTime += CurrDeltaTime;
		ParticleSystem->UpdateTime_Delta = fSaveUpdateDelta;

		// Tick the physics scene
		FPhysScene* PhysScene = World->GetPhysicsScene();
		AWorldSettings * WorldSettings = World->GetWorldSettings();
		check(WorldSettings);
		//@todo phys_thread do we need this?
		World->SetupPhysicsTickFunctions(DeltaTime);
		PhysScene->StartFrame();
		PhysScene->WaitPhysScenes();
		PhysScene->EndFrame();
	}

	// If a vector field module is selected, update the preview visualization.
	if (SelectedModule && SelectedModule->IsA(UParticleModuleVectorFieldLocal::StaticClass()))
	{
		UParticleModuleVectorFieldLocal* VectorFieldModule = (UParticleModuleVectorFieldLocal*)SelectedModule.Get();
		LocalVectorFieldPreviewComponent->VectorField = VectorFieldModule->VectorField;
		LocalVectorFieldPreviewComponent->SetRelativeLocation_Direct(VectorFieldModule->RelativeTranslation);
		LocalVectorFieldPreviewComponent->SetRelativeRotation_Direct(VectorFieldModule->RelativeRotation);
		LocalVectorFieldPreviewComponent->SetRelativeScale3D_Direct(VectorFieldModule->RelativeScale3D);
		LocalVectorFieldPreviewComponent->Intensity = VectorFieldModule->Intensity;
		LocalVectorFieldPreviewComponent->SetVisibleFlag(true);
		LocalVectorFieldPreviewComponent->bHiddenInGame = false;
		LocalVectorFieldPreviewComponent->ReregisterComponent();
	}
	else if(LocalVectorFieldPreviewComponent->GetVisibleFlag())
	{
		LocalVectorFieldPreviewComponent->SetVisibleFlag(false);
		LocalVectorFieldPreviewComponent->ReregisterComponent();
	}

	// If we are doing auto-reset
	if(bIsToggleLoopSystem)
	{
		UParticleSystemComponent* PartComp = ParticleSystemComponent;

		// If system has finish, pause for a bit before resetting.
		if(bIsPendingReset)
		{
			if(TotalTime > ResetTime)
			{
				PartComp->ResetParticles();
				PartComp->ActivateSystem();

				bIsPendingReset = false;
			}
		}
		else
		{
			if(PartComp->HasCompleted())
			{
				bIsPendingReset = true;
				ResetTime = TotalTime + ResetInterval;
			}
		}
	}

	if (bCurrentlySoloing != bIsSoloing)
	{
		bIsSoloing = bCurrentlySoloing;

		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}
	}
}

TStatId FCascade::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCascade, STATGROUP_Tickables);
}

void FCascade::PostUndo(bool bSuccess)
{
	ForceUpdate();
}

TSharedRef<SWidget> FCascade::GenerateAnimSpeedMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_100);
	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_50);
	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_25);
	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_10);
	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_1);

	return MenuBuilder.MakeWidget();
}

void FCascade::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FPropertyChangedEvent Event = PropertyChangedEvent;
	if (SelectedModule)
	{
		SelectedModule->PostEditChangeProperty(Event);
	}
	else if (SelectedEmitter)
	{
		SelectedEmitter->PostEditChangeProperty(Event);
	}
	else if (ParticleSystem)
	{
		ParticleSystem->PostEditChangeProperty(Event);
	}

	OnRestartInLevel();
}

void FCascade::NotifyPreChange(FEditPropertyChain* PropertyChain)
{
	//Needs to stay in-sync with the text in NotifyPostChange
	FText Transaction = NSLOCTEXT("UnrealEd", "CascadePropertyChange", "Change Property");

	BeginTransaction(Transaction);
	ModifyParticleSystem();

	CurveToReplace = NULL;

	// get the property that is being edited
	FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(PropertyChain->GetActiveNode()->GetValue());
	if (ObjProp && 
		(ObjProp->PropertyClass->IsChildOf(UDistributionFloat::StaticClass()) || 
		ObjProp->PropertyClass->IsChildOf(UDistributionVector::StaticClass()))
		)
	{
		UParticleModuleParameterDynamic* DynParamModule = Cast<UParticleModuleParameterDynamic>(SelectedModule);
		if (DynParamModule)
		{
			// Grab the curves...
			DynParamModule->GetCurveObjects(DynParamCurves);
		}
		else
		{
			UObject* EditedObject = NULL;
			if (SelectedModule)
			{
				EditedObject = SelectedModule;
			}
			else
			{
				EditedObject = SelectedEmitter;
			}

			// calculate offset from object to property being edited
			void* BaseObject = EditedObject;
			for (FEditPropertyChain::TIterator It(PropertyChain->GetHead()); It; ++It)
			{
				// don't go past the active property
				if (*It == ObjProp)
				{
					break;
				}

				BaseObject = It->ContainerPtrToValuePtr<void>(BaseObject);

				// If it is an object property, then reset our base pointer/offset
				FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(*It);
				if (ObjectPropertyBase)
				{
					BaseObject = ObjectPropertyBase->GetObjectPropertyValue(BaseObject);
				}
			}

			UObject* ObjPtr = ObjProp->GetObjectPropertyValue(ObjProp->ContainerPtrToValuePtr<void>(BaseObject));
			CurveToReplace = ObjPtr;
		}
	}

	if (SelectedModule)
	{
		if (PropertyChain->GetActiveNode()->GetValue()->GetName() == TEXT("InterpolationMethod"))
		{
			UParticleModuleRequired* ReqMod = Cast<UParticleModuleRequired>(SelectedModule);
			if (ReqMod)
			{
				PreviousInterpolationMethod = (EParticleSubUVInterpMethod)(ReqMod->InterpolationMethod);
			}
		}
	}

}

void FCascade::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyChain)
{
	UParticleModuleParameterDynamic* DynParamModule = Cast<UParticleModuleParameterDynamic>(SelectedModule);
	if (DynParamModule)
	{
		if (DynParamCurves.Num() > 0)
		{
			// Grab the curves...
			TArray<FParticleCurvePair> DPCurves;
			DynParamModule->GetCurveObjects(DPCurves);

			check(DPCurves.Num() == DynParamCurves.Num());
			for (int32 CurveIndex = 0; CurveIndex < DynParamCurves.Num(); CurveIndex++)
			{
				UObject* OldCurve = DynParamCurves[CurveIndex].CurveObject;
				UObject* NewCurve = DPCurves[CurveIndex].CurveObject;
				if (OldCurve != NewCurve)
				{
					ParticleSystem->CurveEdSetup->ReplaceCurve(OldCurve, NewCurve);
					CurveEditor->CurveChanged();
				}
			}
			DynParamCurves.Empty();
		}
	}

	if (CurveToReplace)
	{
		// This should be the same property we just got in NotifyPreChange!
		FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(PropertyChain->GetActiveNode()->GetValue());
		check(ObjProp);
		check(ObjProp->PropertyClass->IsChildOf(UDistributionFloat::StaticClass()) || ObjProp->PropertyClass->IsChildOf(UDistributionVector::StaticClass()));

		UObject* EditedObject = NULL;
		if (SelectedModule)
		{
			EditedObject = SelectedModule;
		}
		else
		{
			EditedObject = SelectedEmitter;
		}

		// calculate offset from object to property being edited
		void* BaseObject = EditedObject;
		for (FEditPropertyChain::TIterator It(PropertyChain->GetHead()); It; ++It)
		{
			// don't go past the active property
			if (*It == ObjProp)
			{
				break;
			}

			BaseObject = It->ContainerPtrToValuePtr<void>(BaseObject);

			// If it is an object property, then reset our base pointer/offset
			FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(*It);
			if (ObjectPropertyBase)
			{
				BaseObject = ObjectPropertyBase->GetObjectPropertyValue(BaseObject);
			}
		}

		UObject* NewCurve = ObjProp->GetObjectPropertyValue(ObjProp->ContainerPtrToValuePtr<void>(BaseObject));

		if (NewCurve)
		{
			ParticleSystem->CurveEdSetup->ReplaceCurve(CurveToReplace, NewCurve);
			CurveEditor->CurveChanged();
		}
	}

	if (SelectedModule || SelectedEmitter)
	{
		if (PropertyChain->GetActiveNode()->GetValue()->GetName() == TEXT("InterpolationMethod"))
		{
			UParticleModuleRequired* ReqMod = Cast<UParticleModuleRequired>(SelectedModule);
			if (ReqMod && SelectedEmitter)
			{
				if (ReqMod->InterpolationMethod != PreviousInterpolationMethod)
				{
					int32 CurrentLODLevel = GetCurrentlySelectedLODLevelIndex();
					if (CurrentLODLevel == 0)
					{
						// The main on is being changed...
						// Check all other LOD levels...
						for (int32 LODIndex = 1; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
						{
							UParticleLODLevel* CheckLOD = SelectedEmitter->LODLevels[LODIndex];
							if (CheckLOD)
							{
								UParticleModuleRequired* CheckReq = CheckLOD->RequiredModule;
								if (CheckReq)
								{
									if (ReqMod->InterpolationMethod == PSUVIM_None)
									{
										CheckReq->InterpolationMethod = PSUVIM_None;
									}
									else
									{
										if (CheckReq->InterpolationMethod == PSUVIM_None)
										{
											CheckReq->InterpolationMethod = ReqMod->InterpolationMethod;
										}
									}
								}
							}
						}
					}
					else
					{
						// The main on is being changed...
						// Check all other LOD levels...
						UParticleLODLevel* CheckLOD = SelectedEmitter->LODLevels[0];
						if (CheckLOD)
						{
							bool bWarn = false;
							UParticleModuleRequired* CheckReq = CheckLOD->RequiredModule;
							if (CheckReq)
							{
								if (ReqMod->InterpolationMethod == PSUVIM_None)
								{
									if (CheckReq->InterpolationMethod != PSUVIM_None)
									{
										ReqMod->InterpolationMethod = PreviousInterpolationMethod;
										bWarn = true;
									}
								}
								else
								{
									if (CheckReq->InterpolationMethod == PSUVIM_None)
									{
										ReqMod->InterpolationMethod = PreviousInterpolationMethod;
										bWarn = true;
									}
								}
							}

							if (bWarn == true)
							{
								FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Cascade_InterpolationMethodLODWarning", "Unable to change InterpolationMethod due to LOD level 0 setting."));
							}
						}
					}
				}
			}
		}

		FPropertyChangedEvent PropertyEvent(PropertyChain->GetActiveNode()->GetValue());
		ParticleSystem->PostEditChangeProperty(PropertyEvent);

		if (SelectedModule)
		{
			if (SelectedModule->IsDisplayedInCurveEd(CurveEditor->GetEdSetup()))
			{
				TArray<FParticleCurvePair> Curves;
				SelectedModule->GetCurveObjects(Curves);

				for (int32 i=0; i<Curves.Num(); i++)
				{
					CurveEditor->GetEdSetup()->ChangeCurveColor(Curves[i].CurveObject, SelectedModule->ModuleEditorColor);
				}
			}
		}
	}

	ParticleSystem->ThumbnailImageOutOfDate = true;

	//Needs to stay in-sync with the text in NotifyPreChange
	FText Transaction = NSLOCTEXT("UnrealEd", "CascadePropertyChange", "Change Property");

	check(bTransactionInProgress);
	EndTransaction( Transaction );

	CurveEditor->CurveChanged();
	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}

	OnRestartInLevel();
}


void FCascade::PreEditCurve(TArray<UObject*> CurvesAboutToChange)
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);

	//Need to keep text in-sync with PostEditCurve
	BeginTransaction( NSLOCTEXT("UnrealEd", "EditCurve", "Edit Curve") );
	ModifyParticleSystem();
	ModifySelectedObjects();

	// Call Modify on all tracks with keys selected
	for (int32 i = 0; i < CurvesAboutToChange.Num(); i++)
	{
		// If this keypoint is from a distribution, call Modify on it to back up its state.
		UDistributionFloat* DistFloat = Cast<UDistributionFloat>(CurvesAboutToChange[i]);
		if (DistFloat)
		{
			DistFloat->SetFlags(RF_Transactional);
			DistFloat->Modify();
		}
		UDistributionVector* DistVector = Cast<UDistributionVector>(CurvesAboutToChange[i]);
		if (DistVector)
		{
			DistVector->SetFlags(RF_Transactional);
			DistVector->Modify();
		}
	}
}

void FCascade::PostEditCurve()
{
	ParticleSystem->BuildEmitters();

	//Need to keep text in-sync with PreEditCurve
	EndTransaction( NSLOCTEXT("UnrealEd", "EditCurve", "Edit Curve") );
}

void FCascade::MovedKey()
{
}

void FCascade::DesireUndo()
{
	OnUndo();
}

void FCascade::DesireRedo()
{
	OnRedo();
}

void FCascade::CreateInternalWidgets()
{
	PreviewViewport =
	SNew(SCascadePreviewViewport)
	.Cascade(SharedThis(this));

	EmitterCanvas = 
	SNew(SCascadeEmitterCanvas)
	.Cascade(SharedThis(this));

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	Details = PropertyModule.CreateDetailView(Args);
	Details->SetObject( ParticleSystem );

	if (!ParticleSystem->CurveEdSetup)
	{
		ParticleSystem->CurveEdSetup = NewObject<UInterpCurveEdSetup>(ParticleSystem, NAME_None, RF_Transactional);
	}

	IDistributionCurveEditorModule* CurveEditorModule = &FModuleManager::LoadModuleChecked<IDistributionCurveEditorModule>( "DistCurveEditor" );
	CurveEditor = CurveEditorModule->CreateCurveEditorWidget(ParticleSystem->CurveEdSetup, this);
}

#define LOCTEXT_NAMESPACE "CascadeToolbar"
void FCascade::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, TSharedRef< FUICommandList > ToolkitCommands, TSharedRef<SWidget> CurrentLOD)
		{
			ToolbarBuilder.BeginSection("CascadeRestart");
			{
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().RestartSimulation);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().RestartInLevel);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("CascadeHistory");
			{
				ToolbarBuilder.AddToolBarButton(FGenericCommands::Get().Undo, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon( FAppStyle::GetAppStyleSetName(), TEXT("Cascade.Undo") ) );
				ToolbarBuilder.AddToolBarButton(FGenericCommands::Get().Redo, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon( FAppStyle::GetAppStyleSetName(), TEXT("Cascade.Redo") ) );
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("CascadeThumbnail");
			{
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().SaveThumbnailImage);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("CascadePreviewOptions");
			{
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().ToggleBounds);
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateStatic(&FCascade::GenerateBoundsMenuContent, ToolkitCommands),
					LOCTEXT( "BoundsMenuCombo_Label", "Bounds Options" ),
					LOCTEXT( "BoundsMenuCombo_ToolTip", "Bounds options"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cascade.ToggleBounds"),
					true
					);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().ToggleOriginAxis);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().CascadeBackgroundColor);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("CascadeLOD");
			{
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().RegenerateLowestLODDuplicatingHighest);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().RegenerateLowestLOD);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().JumpToLowestLOD);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().JumpToLowerLOD);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().AddLODBeforeCurrent);
				ToolbarBuilder.AddSeparator();

				// Show the current and total LODs between the buttons for clearer messaging to the user
				ToolbarBuilder.AddWidget(CurrentLOD);

				ToolbarBuilder.AddSeparator();
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().AddLODAfterCurrent);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().JumpToHigherLOD);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().JumpToHighestLOD);
				ToolbarBuilder.AddToolBarButton(FCascadeCommands::Get().DeleteLOD);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	TSharedRef<SWidget> CurrentLOD = SNew(SBox)
			[
				SNew(SHorizontalBox)
				.AddMetaData<FTagMetaData>(TEXT("Cascade.LODBOx"))
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CurrentLOD", "LOD: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNumericEntryBox<int32>)
					.AllowSpin(true)
					.MinValue(0)
					.MaxValue(this, &FCascade::GetMaxLOD)
					.MinSliderValue(0)
					.MaxSliderValue(this, &FCascade::GetMaxLOD)
					.Value(this, &FCascade::GetCurrentLOD)
					.OnValueChanged(this, &FCascade::OnCurrentLODChanged)
				]
			];

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar, ToolkitCommands, CurrentLOD )
		);

	AddToolbarExtender(ToolbarExtender);

	ICascadeModule* CascadeModule = &FModuleManager::LoadModuleChecked<ICascadeModule>( "Cascade" );
	AddToolbarExtender(CascadeModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}
#undef LOCTEXT_NAMESPACE

void FCascade::BindCommands()
{
	const FCascadeCommands& Commands = FCascadeCommands::Get();

	ToolkitCommands->MapAction(
		Commands.ToggleOriginAxis,
		FExecuteAction::CreateSP(this, &FCascade::OnViewOriginAxis),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewOriginAxisChecked));

	ToolkitCommands->MapAction(
		Commands.View_ParticleCounts,
		FExecuteAction::CreateSP(this, &FCascade::OnViewParticleCounts),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewParticleCountsChecked));

	ToolkitCommands->MapAction(
		Commands.View_ParticleEventCounts,
		FExecuteAction::CreateSP(this, &FCascade::OnViewParticleEventCounts),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewParticleEventCountsChecked));

	ToolkitCommands->MapAction(
		Commands.View_ParticleTimes,
		FExecuteAction::CreateSP(this, &FCascade::OnViewParticleTimes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewParticleTimesChecked));
	
	ToolkitCommands->MapAction(
		Commands.View_ParticleMemory,
		FExecuteAction::CreateSP(this, &FCascade::OnViewParticleMemory),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewParticleMemoryChecked));

	ToolkitCommands->MapAction(
		Commands.View_SystemCompleted,
		FExecuteAction::CreateSP(this, &FCascade::OnViewSystemCompleted),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewSystemCompletedChecked));

	ToolkitCommands->MapAction(
		Commands.View_EmitterTickTimes,
		FExecuteAction::CreateSP(this, &FCascade::OnViewEmitterTickTimes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewEmitterTickTimesChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleGeometry,
		FExecuteAction::CreateSP(this, &FCascade::OnViewGeometry),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewGeometryChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleGeometry_Properties,
		FExecuteAction::CreateSP(this, &FCascade::OnViewGeometryProperties));

	ToolkitCommands->MapAction(
		Commands.ToggleLocalVectorFields,
		FExecuteAction::CreateSP(this, &FCascade::OnViewLocalVectorFields),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewLocalVectorFieldsChecked));

	ToolkitCommands->MapAction(
		Commands.RestartSimulation,
		FExecuteAction::CreateSP(this, &FCascade::OnRestartSimulation));

	ToolkitCommands->MapAction(
		Commands.RestartInLevel,
		FExecuteAction::CreateSP(this, &FCascade::OnRestartInLevel));

	ToolkitCommands->MapAction(
		Commands.SaveThumbnailImage,
		FExecuteAction::CreateSP(this, &FCascade::OnSaveThumbnailImage));

	ToolkitCommands->MapAction(
		Commands.ToggleOrbitMode,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleOrbitMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleOrbitModeChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleMotion,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleMotion),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleMotionChecked));

	ToolkitCommands->MapAction(
		Commands.SetMotionRadius,
		FExecuteAction::CreateSP(this, &FCascade::OnSetMotionRadius));

	ToolkitCommands->MapAction(
		Commands.ViewMode_Wireframe,
		FExecuteAction::CreateSP(this, &FCascade::OnViewMode, VMI_Wireframe),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewModeChecked, VMI_Wireframe));

	ToolkitCommands->MapAction(
		Commands.ViewMode_Unlit,
		FExecuteAction::CreateSP(this, &FCascade::OnViewMode, VMI_Unlit),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewModeChecked, VMI_Unlit));

	ToolkitCommands->MapAction(
		Commands.ViewMode_Lit,
		FExecuteAction::CreateSP(this, &FCascade::OnViewMode, VMI_Lit),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewModeChecked, VMI_Lit));

	ToolkitCommands->MapAction(
		Commands.ViewMode_ShaderComplexity,
		FExecuteAction::CreateSP(this, &FCascade::OnViewMode, VMI_ShaderComplexity),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsViewModeChecked, VMI_ShaderComplexity));

	ToolkitCommands->MapAction(
		Commands.ToggleBounds,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleBounds),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleBoundsChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleBounds_SetFixedBounds,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleBoundsSetFixedBounds));

	ToolkitCommands->MapAction(
		Commands.TogglePostProcess,
		FExecuteAction::CreateSP(this, &FCascade::OnTogglePostProcess),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsTogglePostProcessChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleGrid,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleGridChecked));

	ToolkitCommands->MapAction(
		Commands.CascadePlay,
		FExecuteAction::CreateSP(this, &FCascade::OnPlay),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsPlayChecked));

	ToolkitCommands->MapAction(
		Commands.AnimSpeed_100,
		FExecuteAction::CreateSP(this, &FCascade::OnAnimSpeed, 1.0f),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsAnimSpeedChecked, 1.0f));

	ToolkitCommands->MapAction(
		Commands.AnimSpeed_50,
		FExecuteAction::CreateSP(this, &FCascade::OnAnimSpeed, 0.5f),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsAnimSpeedChecked, 0.5f));

	ToolkitCommands->MapAction(
		Commands.AnimSpeed_25,
		FExecuteAction::CreateSP(this, &FCascade::OnAnimSpeed, 0.25f),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsAnimSpeedChecked, 0.25f));

	ToolkitCommands->MapAction(
		Commands.AnimSpeed_10,
		FExecuteAction::CreateSP(this, &FCascade::OnAnimSpeed, 0.1f),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsAnimSpeedChecked, 0.1f));

	ToolkitCommands->MapAction(
		Commands.AnimSpeed_1,
		FExecuteAction::CreateSP(this, &FCascade::OnAnimSpeed, 0.01f),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsAnimSpeedChecked, 0.01f));

	ToolkitCommands->MapAction(
		Commands.ToggleLoopSystem,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleLoopSystem),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleLoopSystemChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleRealtime,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleRealtime),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleRealtimeChecked));

	ToolkitCommands->MapAction(
		Commands.CascadeBackgroundColor,
		FExecuteAction::CreateSP(this, &FCascade::OnBackgroundColor));

	ToolkitCommands->MapAction(
		Commands.ToggleWireframeSphere,
		FExecuteAction::CreateSP(this, &FCascade::OnToggleWireframeSphere),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsToggleWireframeSphereChecked));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FCascade::OnUndo));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FCascade::OnRedo));

	ToolkitCommands->MapAction(
		Commands.DetailMode_Low,
		FExecuteAction::CreateSP(this, &FCascade::OnDetailMode, DM_Low),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsDetailModeChecked, DM_Low));

	ToolkitCommands->MapAction(
		Commands.DetailMode_Medium,
		FExecuteAction::CreateSP(this, &FCascade::OnDetailMode, DM_Medium),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsDetailModeChecked, DM_Medium));

	ToolkitCommands->MapAction(
		Commands.DetailMode_High,
		FExecuteAction::CreateSP(this, &FCascade::OnDetailMode, DM_High),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsDetailModeChecked, DM_High));

	ToolkitCommands->MapAction(
		Commands.DetailMode_Epic,
		FExecuteAction::CreateSP(this, &FCascade::OnDetailMode, DM_Epic),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsDetailModeChecked, DM_Epic));

	ToolkitCommands->MapAction(
		Commands.Significance_Critical,
		FExecuteAction::CreateSP(this, &FCascade::OnSignificance, EParticleSignificanceLevel::Critical),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsSignificanceChecked, EParticleSignificanceLevel::Critical));

	ToolkitCommands->MapAction(
		Commands.Significance_High,
		FExecuteAction::CreateSP(this, &FCascade::OnSignificance, EParticleSignificanceLevel::High),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsSignificanceChecked, EParticleSignificanceLevel::High));

	ToolkitCommands->MapAction(
		Commands.Significance_Medium,
		FExecuteAction::CreateSP(this, &FCascade::OnSignificance, EParticleSignificanceLevel::Medium),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsSignificanceChecked, EParticleSignificanceLevel::Medium));

	ToolkitCommands->MapAction(
		Commands.Significance_Low,
		FExecuteAction::CreateSP(this, &FCascade::OnSignificance, EParticleSignificanceLevel::Low),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCascade::IsSignificanceChecked, EParticleSignificanceLevel::Low));

	ToolkitCommands->MapAction(
		Commands.RegenerateLowestLODDuplicatingHighest,
		FExecuteAction::CreateSP(this, &FCascade::OnRegenerateLowestLODDuplicatingHighest));

	ToolkitCommands->MapAction(
		Commands.RegenerateLowestLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnRegenerateLowestLOD));

	ToolkitCommands->MapAction(
		Commands.JumpToHighestLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToHighestLOD));

	ToolkitCommands->MapAction(
		Commands.JumpToHigherLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToHigherLOD));

	ToolkitCommands->MapAction(
		Commands.AddLODBeforeCurrent,
		FExecuteAction::CreateSP(this, &FCascade::OnAddLODBeforeCurrent));

	ToolkitCommands->MapAction(
		Commands.AddLODAfterCurrent,
		FExecuteAction::CreateSP(this, &FCascade::OnAddLODAfterCurrent));

	ToolkitCommands->MapAction(
		Commands.JumpToLowerLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLowerLOD));

	ToolkitCommands->MapAction(
		Commands.JumpToLowestLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLowestLOD));

	ToolkitCommands->MapAction(
		Commands.DeleteLOD,
		FExecuteAction::CreateSP(this, &FCascade::OnDeleteLOD));

	ToolkitCommands->MapAction(
		Commands.JumpToLOD0,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLODIndex, 0));

	ToolkitCommands->MapAction(
		Commands.JumpToLOD1,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLODIndex, 1));

	ToolkitCommands->MapAction(
		Commands.JumpToLOD2,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLODIndex, 2));

	ToolkitCommands->MapAction(
		Commands.JumpToLOD3,
		FExecuteAction::CreateSP(this, &FCascade::OnJumpToLODIndex, 3));

	ToolkitCommands->MapAction(
		Commands.DeleteModule,
		FExecuteAction::CreateSP(this, &FCascade::OnDeleteModule, true));

	ToolkitCommands->MapAction(
		Commands.RefreshModule,
		FExecuteAction::CreateSP(this, &FCascade::OnRefreshModule));

	ToolkitCommands->MapAction(
		Commands.SyncMaterial,
		FExecuteAction::CreateSP(this, &FCascade::OnSyncMaterial));

	ToolkitCommands->MapAction(
		Commands.UseMaterial,
		FExecuteAction::CreateSP(this, &FCascade::OnUseMaterial));

	ToolkitCommands->MapAction(
		Commands.DupeFromHigher,
		FExecuteAction::CreateSP(this, &FCascade::OnDupeFromHigher));

	ToolkitCommands->MapAction(
		Commands.ShareFromHigher,
		FExecuteAction::CreateSP(this, &FCascade::OnShareFromHigher));

	ToolkitCommands->MapAction(
		Commands.DupeFromHighest,
		FExecuteAction::CreateSP(this, &FCascade::OnDupeFromHighest));

	ToolkitCommands->MapAction(
		Commands.SetRandomSeed,
		FExecuteAction::CreateSP(this, &FCascade::OnSetRandomSeed));

	ToolkitCommands->MapAction(
		Commands.ConvertToSeeded,
		FExecuteAction::CreateSP(this, &FCascade::OnConvertToSeeded));

	ToolkitCommands->MapAction(
		Commands.RenameEmitter,
		FExecuteAction::CreateSP(this, &FCascade::OnRenameEmitter));

	ToolkitCommands->MapAction(
		Commands.DuplicateEmitter,
		FExecuteAction::CreateSP(this, &FCascade::OnDuplicateEmitter, false));

	ToolkitCommands->MapAction(
		Commands.DuplicateShareEmitter,
		FExecuteAction::CreateSP(this, &FCascade::OnDuplicateEmitter, true));

	ToolkitCommands->MapAction(
		Commands.DeleteEmitter,
		FExecuteAction::CreateSP(this, &FCascade::OnDeleteEmitter));

	ToolkitCommands->MapAction(
		Commands.ExportEmitter,
		FExecuteAction::CreateSP(this, &FCascade::OnExportEmitter));

	ToolkitCommands->MapAction(
		Commands.ExportAllEmitters,
		FExecuteAction::CreateSP(this, &FCascade::OnExportAll));

	ToolkitCommands->MapAction(
		Commands.SelectParticleSystem,
		FExecuteAction::CreateSP(this, &FCascade::OnSelectParticleSystem));

	ToolkitCommands->MapAction(
		Commands.NewEmitterBefore,
		FExecuteAction::CreateSP(this, &FCascade::OnNewEmitterBefore));

	ToolkitCommands->MapAction(
		Commands.NewEmitterAfter,
		FExecuteAction::CreateSP(this, &FCascade::OnNewEmitterAfter));

	ToolkitCommands->MapAction(
		Commands.RemoveDuplicateModules,
		FExecuteAction::CreateSP(this, &FCascade::OnRemoveDuplicateModules));
}

void FCascade::InitParticleModuleClasses()
{
	if (bParticleModuleClassesInitialized)
		return;

	for(TObjectIterator<UClass> It; It; ++It)
	{
		// Find all ParticleModule classes (ignoring abstract or ParticleTrailModule classes
		if (It->IsChildOf(UParticleModule::StaticClass()))
		{
			if (!(It->HasAnyClassFlags(CLASS_Abstract)))
			{
				ParticleModuleClasses.Add(*It);
			}
			else
			{
				ParticleModuleBaseClasses.Add(*It);
			}
		}
	}

	bParticleModuleClassesInitialized = true;
}

TOptional<int32> FCascade::GetMaxLOD() const
{
	int32 LODCount = ParticleSystem ? (ParticleSystem->Emitters.Num() > 0) ? (ParticleSystem->Emitters[0]->LODLevels.Num() > 0 ) ? ParticleSystem->Emitters[0]->LODLevels.Num() - 1 : 0 : 0 : 0;
	return LODCount;
}

TOptional<int32> FCascade::GetCurrentLOD() const
{
	return CurrentLODIdx;
}

void FCascade::OnCurrentLODChanged(int32 NewLOD)
{
	SetLODValue(NewLOD);
	
	if (PreviewViewport.IsValid())
	{
		PreviewViewport->RefreshViewport();
	}

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::MotionRadiusCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		MotionModeRadius = atof(TCHAR_TO_ANSI( *CommentText.ToString() ));
	}

	CloseEntryPopup();
}

void FCascade::SphereRadiusCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		PreviewViewport->GetViewportClient()->GetWireSphereRadius() = atof(TCHAR_TO_ANSI(*CommentText.ToString()));
		ToggleDrawOption(FCascadeEdPreviewViewportClient::WireSphere);
	}

	CloseEntryPopup();
}

void FCascade::EmitterNameCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	if ((CommitInfo == ETextCommit::OnEnter) && SelectedEmitter)
	{
		FText Transaction = NSLOCTEXT("UnrealEd", "EmitterRename", "Rename Emitter");

		BeginTransaction( Transaction );

		ParticleSystem->PreEditChange(NULL);
		ParticleSystemComponent->PreEditChange(NULL);

		SelectedEmitter->Modify();
		SelectedEmitter->SetEmitterName( *CommentText.ToString() );
		
		ParticleSystemComponent->PostEditChange();
		ParticleSystem->PostEditChange();

		EndTransaction( Transaction );

		// Refresh viewport
		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}
	}

	CloseEntryPopup();
}

void FCascade::UpdateLODLevel()
{
	int32 CurrentLODLevel = GetCurrentlySelectedLODLevelIndex();
	SetLODValue(CurrentLODLevel);
}

void FCascade::SetLODValue(int32 LODSetting)
{
	if (LODSetting >= 0)
	{
		if (ParticleSystem)
		{
			ParticleSystem->EditorLODSetting = LODSetting;
		}
		if (ParticleSystemComponent)
		{
			const int32 OldEditorLODLevel = ParticleSystemComponent->EditorLODLevel;
			ParticleSystemComponent->EditorLODLevel = LODSetting;
			ParticleSystemComponent->SetLODLevel(LODSetting);
		}

		CurrentLODIdx = LODSetting;
	}

	if (!GEngine->bEnableEditorPSysRealtimeLOD && ParticleSystemComponent != NULL)
	{
		for (TObjectIterator<UParticleSystemComponent> It;It;++It)
		{
			if (It->Template == ParticleSystemComponent->Template)
			{
				It->EditorLODLevel = LODSetting;
				It->SetLODLevel(LODSetting);
			}
		}
	}
}

void FCascade::ReassociateParticleSystem() const
{
	if (ParticleSystemComponent)
	{
		if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
		{
			PreviewViewport->GetViewportClient()->GetPreviewScene().RemoveComponent(ParticleSystemComponent);
			PreviewViewport->GetViewportClient()->GetPreviewScene().AddComponent(ParticleSystemComponent, FTransform::Identity);
		}
	}
}

void FCascade::RestartParticleSystem()
{
	if (ParticleSystemComponent)
	{
		ParticleSystemComponent->ResetParticles();
		ParticleSystemComponent->SetManagingSignificance(true);
		ParticleSystemComponent->SetRequiredSignificance(RequiredSignificance);
		ParticleSystemComponent->ActivateSystem();
		if (ParticleSystemComponent->Template)
		{
			ParticleSystemComponent->Template->bShouldResetPeakCounts = true;
		}
		ParticleSystemComponent->bIsViewRelevanceDirty = true;
		ParticleSystemComponent->CachedViewRelevanceFlags.Empty();
		ParticleSystemComponent->ConditionalCacheViewRelevanceFlags();

		ReassociateParticleSystem();
	}

	if (ParticleSystem)
	{
		ParticleSystem->CalculateMaxActiveParticleCounts();
	}

	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->UpdateMemoryInformation();
	}

	PreviewViewport->RefreshViewport();
}

void FCascade::ForceUpdate()
{
	// Touch the module lists in each emitter.
	ParticleSystem->UpdateAllModuleLists();
	UpdateLODLevel();
	ParticleSystemComponent->ResetParticles();
	ParticleSystemComponent->InitializeSystem();
	
	// 'Refresh' the viewport
	if (PreviewViewport.IsValid())
	{
		PreviewViewport->RefreshViewport();
	}

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}

	if (CurveEditor.IsValid())
	{
		CurveEditor->CurveChanged();
	}

	if( Details.IsValid() )
	{
		const TArray< TWeakObjectPtr<UObject> > NewSelectedObjects = Details->GetSelectedObjects();
		Details->SetObjects( NewSelectedObjects, true );
	}
}

bool FCascade::PromptForCancellingSoloingMode( const FText& InOperationDesc)
{
	FText DisplayMessage = FText::Format( NSLOCTEXT("UnrealEd", "CASCADE_CancelSoloing", "Disable soloing to perform the following:\n{0}"), InOperationDesc );

	bool bCancelSoloing = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, DisplayMessage);
	if (bCancelSoloing)
	{
		ParticleSystem->TurnOffSoloing();
		bIsSoloing = false;
	}

	ForceUpdate();
	return bCancelSoloing;
}

bool FCascade::DuplicateEmitter(UParticleEmitter* SourceEmitter, UParticleSystem* DestSystem, bool bShare)
{
	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( NSLOCTEXT("UnrealEd", "DuplicateEmitter", "Duplicate Emitter") ) == false)
		{
			return false;
		}
	}

	UObject* SourceOuter = SourceEmitter->GetOuter();
	if (SourceOuter != DestSystem)
	{
		if (bShare == true)
		{
			UE_LOGF(LogCascade, Warning, "Can't share modules across particle systems!");
			bShare = false;
		}
	}

	int32 InsertionIndex = -1;
	if (SourceOuter == DestSystem)
	{
		UParticleSystem* SourcePSys = Cast<UParticleSystem>(SourceOuter);
		if (SourcePSys)
		{
			// Find the source emitter in the SourcePSys emitter array
			for (int32 CheckSourceIndex = 0; CheckSourceIndex < SourcePSys->Emitters.Num(); CheckSourceIndex++)
			{
				if (SourcePSys->Emitters[CheckSourceIndex] == SourceEmitter)
				{
					InsertionIndex = CheckSourceIndex + 1;
					break;
				}
			}
		}
	}

	// Find desired class of new module.
	UClass* NewEmitClass = SourceEmitter->GetClass();
	if (NewEmitClass == UParticleSpriteEmitter::StaticClass())
	{
		// Construct it
		UParticleEmitter* NewEmitter = NewObject<UParticleEmitter>(DestSystem, NewEmitClass, NAME_None, RF_Transactional);

		check(NewEmitter);

		FString	NewName = SourceEmitter->GetEmitterName().ToString();
		NewEmitter->SetEmitterName(FName(*NewName));
		NewEmitter->EmitterEditorColor = FColor::MakeRandomColor();
		NewEmitter->EmitterEditorColor.A = 255;

		NewEmitter->InitialAllocationCount = SourceEmitter->InitialAllocationCount;
		NewEmitter->QualityLevelSpawnRateScale = SourceEmitter->QualityLevelSpawnRateScale;
		NewEmitter->DetailModeBitmask = SourceEmitter->DetailModeBitmask;
		NewEmitter->DetailModeDisplay = SourceEmitter->DetailModeDisplay;
		NewEmitter->bDisabledLODsKeepEmitterAlive = SourceEmitter->bDisabledLODsKeepEmitterAlive;
		NewEmitter->bUseLegacySpawningBehavior = SourceEmitter->bUseLegacySpawningBehavior;
		NewEmitter->bDisableWhenInsignficant = SourceEmitter->bDisableWhenInsignficant;
		NewEmitter->SignificanceLevel = SourceEmitter->SignificanceLevel;
		NewEmitter->EmitterRenderMode = SourceEmitter->EmitterRenderMode;
		NewEmitter->bCollapsed = false;

		//	'Private' data - not required by the editor
		UObject*			DupObject;
		UParticleLODLevel*	SourceLODLevel;
		UParticleLODLevel*	NewLODLevel;
		UParticleLODLevel*	PrevSourceLODLevel = NULL;
		UParticleLODLevel*	PrevLODLevel = NULL;

		NewEmitter->LODLevels.InsertZeroed(0, SourceEmitter->LODLevels.Num());
		for (int32 LODIndex = 0; LODIndex < SourceEmitter->LODLevels.Num(); LODIndex++)
		{
			SourceLODLevel	= SourceEmitter->LODLevels[LODIndex];
			NewLODLevel = NewObject<UParticleLODLevel>(NewEmitter, NAME_None, RF_Transactional);
			check(NewLODLevel);

			NewLODLevel->Level					= SourceLODLevel->Level;
			NewLODLevel->bEnabled				= SourceLODLevel->bEnabled;

			// The RequiredModule
			if (bShare)
			{
				NewLODLevel->RequiredModule = SourceLODLevel->RequiredModule;
			}
			else
			{
				if ((LODIndex > 0) && (PrevSourceLODLevel->RequiredModule == SourceLODLevel->RequiredModule))
				{
					PrevLODLevel->RequiredModule->LODValidity |= (1 << LODIndex);
					NewLODLevel->RequiredModule = PrevLODLevel->RequiredModule;
				}
				else
				{
					DupObject = StaticDuplicateObject(SourceLODLevel->RequiredModule, DestSystem);
					check(DupObject);
					NewLODLevel->RequiredModule						= Cast<UParticleModuleRequired>(DupObject);
					NewLODLevel->RequiredModule->ModuleEditorColor	= FColor::MakeRandomColor();
					NewLODLevel->RequiredModule->LODValidity		= (1 << LODIndex);
				}
			}

			// The SpawnModule
			if (bShare)
			{
				NewLODLevel->SpawnModule = SourceLODLevel->SpawnModule;
			}
			else
			{
				if ((LODIndex > 0) && (PrevSourceLODLevel->SpawnModule == SourceLODLevel->SpawnModule))
				{
					PrevLODLevel->SpawnModule->LODValidity |= (1 << LODIndex);
					NewLODLevel->SpawnModule = PrevLODLevel->SpawnModule;
				}
				else
				{
					DupObject = StaticDuplicateObject(SourceLODLevel->SpawnModule, DestSystem);
					check(DupObject);
					NewLODLevel->SpawnModule					= Cast<UParticleModuleSpawn>(DupObject);
					NewLODLevel->SpawnModule->ModuleEditorColor	= FColor::MakeRandomColor();
					NewLODLevel->SpawnModule->LODValidity		= (1 << LODIndex);
				}
			}

			// Copy each module
			NewLODLevel->Modules.InsertZeroed(0, SourceLODLevel->Modules.Num());
			for (int32 ModuleIndex = 0; ModuleIndex < SourceLODLevel->Modules.Num(); ModuleIndex++)
			{
				UParticleModule* SourceModule = SourceLODLevel->Modules[ModuleIndex];
				if (bShare)
				{
					NewLODLevel->Modules[ModuleIndex] = SourceModule;
				}
				else
				{
					if ((LODIndex > 0) && (PrevSourceLODLevel->Modules[ModuleIndex] == SourceLODLevel->Modules[ModuleIndex]))
					{
						PrevLODLevel->Modules[ModuleIndex]->LODValidity |= (1 << LODIndex);
						NewLODLevel->Modules[ModuleIndex] = PrevLODLevel->Modules[ModuleIndex];
					}
					else
					{
						DupObject = StaticDuplicateObject(SourceModule, DestSystem);
						if (DupObject)
						{
							UParticleModule* Module				= Cast<UParticleModule>(DupObject);
							Module->ModuleEditorColor			= FColor::MakeRandomColor();
							NewLODLevel->Modules[ModuleIndex]	= Module;
						}
					}
				}
			}

			// TypeData module as well...
			if (SourceLODLevel->TypeDataModule)
			{
				if (bShare)
				{
					NewLODLevel->TypeDataModule = SourceLODLevel->TypeDataModule;
				}
				else
				{
					if ((LODIndex > 0) && (PrevSourceLODLevel->TypeDataModule == SourceLODLevel->TypeDataModule))
					{
						PrevLODLevel->TypeDataModule->LODValidity |= (1 << LODIndex);
						NewLODLevel->TypeDataModule = PrevLODLevel->TypeDataModule;
					}
					else
					{
						DupObject = StaticDuplicateObject(SourceLODLevel->TypeDataModule, DestSystem);
						if (DupObject)
						{
							UParticleModule* Module		= Cast<UParticleModule>(DupObject);
							Module->ModuleEditorColor	= FColor::MakeRandomColor();
							NewLODLevel->TypeDataModule = CastChecked<UParticleModuleTypeDataBase>(Module);
						}
					}
				}
			}
			NewLODLevel->ConvertedModules		= true;
			NewLODLevel->PeakActiveParticles	= SourceLODLevel->PeakActiveParticles;

			NewEmitter->LODLevels[LODIndex]		= NewLODLevel;

			PrevLODLevel = NewLODLevel;
			PrevSourceLODLevel = SourceLODLevel;
		}

		// Generate all the levels that are present in other emitters...
		// NOTE: Big assumptions - the highest and lowest are 0,100 respectively and they MUST exist.
		if (DestSystem->Emitters.Num() > 0)
		{
			int32 DestLODCount = 0;
			int32 NewLODCount = 0;

			{
				UParticleEmitter* DestEmitter = DestSystem->Emitters[0];
				DestLODCount = DestEmitter->LODLevels.Num();
				NewLODCount = NewEmitter->LODLevels.Num();
			}

			if (DestLODCount != NewLODCount)
			{
				UE_LOGF(LogCascade, Log, "Generating existing LOD levels...");

				if (DestLODCount < NewLODCount)
				{
					for (int32 DestEmitIndex = 0; DestEmitIndex < DestSystem->Emitters.Num(); DestEmitIndex++)
					{
						UParticleEmitter* DestEmitter = DestSystem->Emitters[DestEmitIndex];
						for (int32 InsertIndex = DestLODCount; InsertIndex < NewLODCount; InsertIndex++)
						{
							DestEmitter->CreateLODLevel(InsertIndex);
						}
						DestEmitter->UpdateModuleLists();
					}
				}
				else
				{
					for (int32 InsertIndex = NewLODCount; InsertIndex < DestLODCount; InsertIndex++)
					{
						NewEmitter->CreateLODLevel(InsertIndex);
					}
				}
			}
		}

		NewEmitter->UpdateModuleLists();

		// Add to selected emitter
		if ((InsertionIndex >= 0) && (InsertionIndex < DestSystem->Emitters.Num()))
		{
			DestSystem->Emitters.Insert(NewEmitter, InsertionIndex);
		}
		else
		{
			DestSystem->Emitters.Add(NewEmitter);
		}
	}
	else
	{
		FText Message = FText::Format( NSLOCTEXT("UnrealEd", "Prompt_4", "{0} support coming soon."), FText::FromString( NewEmitClass->GetDesc() ) );
		FMessageDialog::Open(EAppMsgType::Ok, Message);
		return false;
	}

	DestSystem->SetupSoloing();

	return true;
}

void FCascade::MoveSelectedEmitter(int32 MoveAmount)
{
	if (!SelectedEmitter)
		return;

	FText Transaction = NSLOCTEXT("UnrealEd", "MoveEmitter", "Move Selected Emitter");

	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( Transaction ) == false)
		{
			return;
		}
	}

	BeginTransaction( Transaction );
	ModifyParticleSystem();

	int32 CurrentEmitterIndex = ParticleSystem->Emitters.Find(SelectedEmitter);
	check(CurrentEmitterIndex != INDEX_NONE);

	int32 NewEmitterIndex = FMath::Clamp<int32>(CurrentEmitterIndex + MoveAmount, 0, ParticleSystem->Emitters.Num() - 1);

	if (NewEmitterIndex != CurrentEmitterIndex)
	{
		ParticleSystem->PreEditChange(NULL);

		ParticleSystem->Emitters.Remove(SelectedEmitter);
		ParticleSystem->Emitters.InsertZeroed(NewEmitterIndex);
		ParticleSystem->Emitters[NewEmitterIndex] = SelectedEmitter;

		ParticleSystem->PostEditChange();

		ParticleSystem->SetupSoloing();

		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}
	}

	EndTransaction( Transaction );

	ParticleSystem->MarkPackageDirty();
}

void FCascade::AddNewEmitter(int32 PositionOffset)
{
	if (SelectedEmitter == NULL)
	{
		return;
	}

	int32 EmitterCount = ParticleSystem->Emitters.Num();
	int32 EmitterIndex = -1;
	for (int32 Index = 0; Index < EmitterCount; Index++)
	{
		UParticleEmitter* CheckEmitter = ParticleSystem->Emitters[Index];
		if (SelectedEmitter == CheckEmitter)
		{
			EmitterIndex = Index;
			break;
		}
	}

	if (EmitterIndex != -1)
	{
		UE_LOGF(LogCascade, Log, "Insert New Emitter Before %d", EmitterIndex);

		// Fake create it at the end
		OnNewEmitter();

		if (EmitterCount + 1 == ParticleSystem->Emitters.Num())
		{
			UParticleEmitter* NewEmitter = ParticleSystem->Emitters[EmitterCount];
			SetSelectedEmitter(NewEmitter);
			if ((PositionOffset == 0) || (EmitterIndex + PositionOffset < EmitterCount))
			{
				MoveSelectedEmitter(EmitterIndex - EmitterCount + PositionOffset);
			}
		}
	}
}

void FCascade::DuplicateModule(bool bDoShare, bool bUseHighest)
{
	if (!SelectedModule && !SelectedEmitter)
	{
		return;
	}

	int32	CurrLODSetting	= GetCurrentlySelectedLODLevelIndex();
	if (SelectedEmitter->IsLODLevelValid(CurrLODSetting) == false)
	{
		return;
	}

	if (CurrLODSetting == 0)
	{
		// High LOD modules don't allow this.
		return;
	}

	UParticleLODLevel* SourceLODLevel = SelectedEmitter->GetLODLevel(bUseHighest? 0: CurrLODSetting - 1);
	check(SourceLODLevel);
	UParticleModule* HighModule = SourceLODLevel->GetModuleAtIndex(SelectedModuleIndex);
	if (HighModule == NULL)
	{
		// Couldn't find the highest module???
		return;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "DupeSelectedModule", "Duplicate Selected Module");

	BeginTransaction( Transaction );
	ModifySelectedObjects();
	ModifyParticleSystem();

	ParticleSystem->PreEditChange(NULL);

	UParticleModule* NewModule = NULL;
	UParticleLODLevel* DestLODLevel;

	bool bIsShared = GetIsModuleShared(SelectedModule);
	// Store the index of the selected module...
	// Force copy the source module...
	DestLODLevel = SelectedEmitter->GetLODLevel(CurrLODSetting);
	NewModule = HighModule->GenerateLODModule(SourceLODLevel, DestLODLevel, 100.0f, false, !bDoShare);
	check(NewModule);

	for (int32 LODIndex = CurrLODSetting; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
	{
		DestLODLevel = SelectedEmitter->GetLODLevel(LODIndex);
		if (SelectedModule->IsUsedInLODLevel(LODIndex))
		{
			if (bIsShared == false)
			{
				// Turn off the LOD validity in the original module...
				int32 LODIndexToUse = bDoShare? DestLODLevel->Level: LODIndex;
				SelectedModule->LODValidity &= ~(1 << LODIndexToUse);
			}
			// Turn on the LOD validity in the new module...
			NewModule->LODValidity |= (1 << LODIndex);

			// Store the new module
			switch (SelectedModuleIndex)
			{
			case INDEX_NONE:
				break;
			case INDEX_REQUIREDMODULE:
				DestLODLevel->RequiredModule = CastChecked<UParticleModuleRequired>(NewModule);
				break;
			case INDEX_SPAWNMODULE:
				DestLODLevel->SpawnModule = CastChecked<UParticleModuleSpawn>(NewModule);
				break;
			case INDEX_TYPEDATAMODULE:
				DestLODLevel->TypeDataModule = CastChecked<UParticleModuleTypeDataBase>(NewModule);
				break;
			default:
				DestLODLevel->Modules[SelectedModuleIndex] = NewModule;
				break;
			}
		}
	}

	SelectedModule = NewModule;
	if (SelectedEmitter)
	{
		SelectedEmitter->UpdateModuleLists();
	}

	ParticleSystem->PostEditChange();

	SetSelectedModule(SelectedEmitter, SelectedModule);

	EndTransaction( Transaction );
	ForceUpdate();

	ParticleSystem->MarkPackageDirty();

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::ExportSelectedEmitter()
{
	if (!SelectedEmitter)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_NoEmitterSelectedForExport", "No emitter selected for export"));
		return;
	}
	
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
	for (FSelectionIterator Itor(*GEditor->GetSelectedObjects()) ; Itor ; ++Itor)
	{
		UParticleSystem* DestPartSys = Cast<UParticleSystem>(*Itor);
		if (DestPartSys && (DestPartSys != ParticleSystem))
		{
			int32 NewCount = 0;
			if (DestPartSys->Emitters.Num() > 0)
			{
				UParticleEmitter* DestEmitter0 = DestPartSys->Emitters[0];

				NewCount = DestEmitter0->LODLevels.Num() - SelectedEmitter->LODLevels.Num();
				if (NewCount > 0)
				{
					// There are more LODs in the destination than the source... Add enough to cover.
					int32 StartIndex = SelectedEmitter->LODLevels.Num();
					for (int32 InsertIndex = 0; InsertIndex < NewCount; InsertIndex++)
					{
						SelectedEmitter->CreateLODLevel(StartIndex + InsertIndex, true);
					}
					SelectedEmitter->UpdateModuleLists();
				}
				else if (NewCount < 0)
				{
					int32 InsertCount = -NewCount;
					// There are fewer LODs in the destination than the source... Add enough to cover.
					int32 StartIndex = DestEmitter0->LODLevels.Num();
					for (int32 EmitterIndex = 0; EmitterIndex < DestPartSys->Emitters.Num(); EmitterIndex++)
					{
						UParticleEmitter* DestEmitter = DestPartSys->Emitters[EmitterIndex];
						if (DestEmitter)
						{
							for (int32 InsertIndex = 0; InsertIndex < InsertCount; InsertIndex++)
							{
								DestEmitter->CreateLODLevel(StartIndex + InsertIndex, false);
							}
							DestEmitter->UpdateModuleLists();
						}
					}

					// Add the slots in the LODDistances array
					DestPartSys->LODDistances.AddZeroed(InsertCount);
					for (int32 DistIndex = StartIndex; DistIndex < DestPartSys->LODDistances.Num(); DistIndex++)
					{
						DestPartSys->LODDistances[DistIndex] = DistIndex * 2500.0f;
					}
					DestPartSys->LODSettings.AddZeroed(InsertCount);
					for (int32 DistIndex = StartIndex; DistIndex < DestPartSys->LODSettings.Num(); DistIndex++)
					{
						DestPartSys->LODSettings[DistIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
					}
				}
			}
			else
			{
				int32 InsertCount = SelectedEmitter->LODLevels.Num();
				// Reset LODSettings and LODDistances arrays
				DestPartSys->LODSettings.Empty();
				DestPartSys->LODDistances.Empty();
				DestPartSys->LODDistances.AddZeroed(InsertCount);
				for (int32 DistIndex = 0; DistIndex < InsertCount; DistIndex++)
				{
					DestPartSys->LODDistances[DistIndex] = DistIndex * 2500.0f;
				}
				DestPartSys->LODSettings.AddZeroed(InsertCount);
				for (int32 DistIndex = 0; DistIndex < InsertCount; DistIndex++)
				{
					DestPartSys->LODSettings[DistIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
				}
			}

			ParticleSystem->SetupSoloing();		// we may have changed the number of LODs, so our soloing information could be invalid


			if (!DuplicateEmitter(SelectedEmitter, DestPartSys, false))
			{
				FText Message = FText::Format( NSLOCTEXT("UnrealEd", "Error_FailedToCopyFormatting", "Failed to copy {0} to {1}"), 
					FText::FromName( SelectedEmitter->GetEmitterName() ),
					FText::FromString( DestPartSys->GetName() ) );

				FMessageDialog::Open(EAppMsgType::Ok, Message);
			}

			DestPartSys->MarkPackageDirty();

			// If we temporarily inserted LOD levels into the selected emitter,
			// we need to remove them now...
			if (NewCount > 0)
			{
				int32 CurrCount = SelectedEmitter->LODLevels.Num();
				for (int32 RemoveIndex = CurrCount - 1; RemoveIndex >= (CurrCount - NewCount); RemoveIndex--)
				{
					SelectedEmitter->LODLevels.RemoveAt(RemoveIndex);
				}
				SelectedEmitter->UpdateModuleLists();
			}

			// Find instances of this particle system and reset them...
			for (TObjectIterator<UParticleSystemComponent> It;It;++It)
			{
				if (It->Template == DestPartSys)
				{
					UParticleSystemComponent* PSysComp = *It;

					// If the preview window the system component belonged to has been
					// destroyed, but garbage collection has not yet run, we will be
					// able to find the system but it won't have a world nor does it need
					// to be reactivated
					if (PSysComp->GetWorld())
					{
						// Force a recache of the view relevance
						PSysComp->bIsViewRelevanceDirty = true;
						bool bIsActive = PSysComp->IsActive();
						PSysComp->DeactivateSystem();
						PSysComp->ResetParticles();
						if (bIsActive)
						{
							PSysComp->ActivateSystem();
						}
						PSysComp->ReregisterComponent();
					}
				}
			}

			ICascadeModule* CascadeModule = &FModuleManager::GetModuleChecked<ICascadeModule>("Cascade");
			CascadeModule->RefreshCascade(DestPartSys);
		}
	}
}

void FCascade::RegenerateLowestLOD(bool bDupeHighest)
{
	if ((ParticleSystem == NULL) || (ParticleSystem->Emitters.Num() == 0))
	{
		return;
	}

	ParticleSystem->bRegenerateLODDuplicate = bDupeHighest;

	const FText	WarningMessage = NSLOCTEXT("UnrealEd", "CascadeRegenLowLODWarningLine1", "*** WARNING ***\nRegenerating the lowest LOD level will delete\nall other LOD levels from the particle system!\nAre you sure you want to do so?");

	if ( EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage) )
	{
		UE_LOGF(LogCascade, Log, "Regenerate Lowest LOD levels!");

		FText Transaction = NSLOCTEXT("UnrealEd", "CascadeRegenerateLowestLOD", "Regenerate Lowest LOD");

		BeginTransaction( Transaction );
		ModifyParticleSystem(true);

		// Delete all LOD levels from each emitter...
		for (int32 EmitterIndex = 0; EmitterIndex < ParticleSystem->Emitters.Num(); EmitterIndex++)
		{
			UParticleEmitter*	Emitter	= ParticleSystem->Emitters[EmitterIndex];
			if (Emitter)
			{
				for (int32 LODIndex = Emitter->LODLevels.Num() - 1; LODIndex > 0; LODIndex--)
				{
					Emitter->LODLevels.RemoveAt(LODIndex);
				}
				if (Emitter->AutogenerateLowestLODLevel(ParticleSystem->bRegenerateLODDuplicate) == false)
				{
					UE_LOGF(LogCascade, Warning, "Failed to autogenerate lowest LOD level!");
				}

				Emitter->UpdateModuleLists();
			}
		}

		// Reset the LOD distances
		ParticleSystem->LODDistances.Empty();
		ParticleSystem->LODSettings.Empty();
		UParticleEmitter* SourceEmitter = ParticleSystem->Emitters[0];
		if (SourceEmitter)
		{
			ParticleSystem->LODDistances.AddUninitialized(SourceEmitter->LODLevels.Num());
			for (int32 LODIndex = 0; LODIndex < ParticleSystem->LODDistances.Num(); LODIndex++)
			{
				ParticleSystem->LODDistances[LODIndex] = LODIndex * 2500.0f;
			}
			ParticleSystem->LODSettings.AddUninitialized(SourceEmitter->LODLevels.Num());
			for (int32 LODIndex = 0; LODIndex < ParticleSystem->LODSettings.Num(); LODIndex++)
			{
				ParticleSystem->LODSettings[LODIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
			}
		}

		ParticleSystem->SetupSoloing();

		OnRestartInLevel();

		check(bTransactionInProgress);
		EndTransaction( Transaction );

		// Re-fill the LODCombo so that deleted LODLevels are removed.
		if (PreviewViewport.IsValid())
		{
			PreviewViewport->RefreshViewport();
		}

		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}

		if (ParticleSystemComponent)
		{
			ParticleSystemComponent->ResetParticles();
			ParticleSystemComponent->InitializeSystem();
		}
	}
	else
	{
		UE_LOGF(LogCascade, Log, "CANCELLED Regenerate Lowest LOD levels!");
	}

	UpdateLODLevel();
}

void FCascade::AddLOD(bool bBeforeCurrent)
{
	if (bIsSoloing == true)
	{
		FText Description = bBeforeCurrent ? NSLOCTEXT("UnrealEd", "CascadeLODAddBefore", "Add LOD Before Current") : NSLOCTEXT("UnrealEd", "CascadeLODAddAfter", "Add LOD After Current");
		if (PromptForCancellingSoloingMode( Description ) == false)
		{
			return;
		}
	}

	// See if there is already a LOD level for this value...
	if (ParticleSystem->Emitters.Num() > 0)
	{
		UParticleEmitter* FirstEmitter = ParticleSystem->Emitters[0];
		if (FirstEmitter)
		{
			if (FirstEmitter->LODLevels.Num() >= 8)
			{
				FNotificationInfo Info( NSLOCTEXT("UnrealEd", "CascadeTooManyLODs", "Max LOD levels (8) already present") );
				Info.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
				return;
			}
		}

		int32 CurrentLODIndex = GetCurrentlySelectedLODLevelIndex();
		if (bBeforeCurrent && CurrentLODIndex < 0)
		{
			return;
		}
		else if (!bBeforeCurrent)
		{
			CurrentLODIndex++;
		}

		UE_LOGF(LogCascade, Log, "Inserting LOD level at %d", CurrentLODIndex);

		FText Transaction = NSLOCTEXT("UnrealEd", "CascadeLODAdd", "Add LOD");

		BeginTransaction( Transaction );
		ModifyParticleSystem(true);

		for (int32 EmitterIndex = 0; EmitterIndex < ParticleSystem->Emitters.Num(); EmitterIndex++)
		{
			UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIndex];
			if (Emitter)
			{
				Emitter->CreateLODLevel(CurrentLODIndex);
			}
		}

		//This should probably have fixed size and behave like LODDistances but for now just avoid the crash.
		ParticleSystem->LODSettings.SetNumZeroed(FMath::Max(CurrentLODIndex, ParticleSystem->LODSettings.Num()));

		ParticleSystem->LODDistances.InsertZeroed(CurrentLODIndex, 1);
		if (CurrentLODIndex == 0)
		{
			ParticleSystem->LODDistances[CurrentLODIndex] = 0.0f;
		}
		else
		{
			ParticleSystem->LODDistances[CurrentLODIndex] = ParticleSystem->LODDistances[CurrentLODIndex - 1];
		}

		ParticleSystem->LODSettings.InsertZeroed(CurrentLODIndex, 1);
		if (CurrentLODIndex == 0)
		{
			ParticleSystem->LODSettings[CurrentLODIndex] = FParticleSystemLOD::CreateParticleSystemLOD();
		}
		else
		{
			ParticleSystem->LODSettings[CurrentLODIndex] = ParticleSystem->LODSettings[CurrentLODIndex - 1];
		}

		ParticleSystem->SetupSoloing();

		check(bTransactionInProgress);
		EndTransaction( Transaction );

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.NewLOD"), FAnalyticsEventAttribute(TEXT("Index"), CurrentLODIndex));
		}

		UpdateLODLevel();
		SetSelectedModule(SelectedEmitter, SelectedModule);
		ForceUpdate();

		OnRestartInLevel();
	}
}

void FCascade::SetSelectedInCurveEditor()
{
	if (!CurveEditor.IsValid())
	{
		return;
	}

	CurveEditor->ClearAllSelectedCurves();
	if (SelectedModule)
	{
		TArray<FParticleCurvePair> Curves;
		SelectedModule->GetCurveObjects(Curves);
		for (int32 CurveIndex = 0; CurveIndex < Curves.Num(); CurveIndex++)
		{
			UObject* Distribution = Curves[CurveIndex].CurveObject;
			if (Distribution)
			{
				CurveEditor->SetCurveSelected(Distribution, true);
			}
		}
		CurveEditor->SetActiveTabToFirstSelected();
		CurveEditor->ScrollToFirstSelected();
	}
	CurveEditor->RefreshViewport();
}

bool FCascade::BeginTransaction(const FText& Description)
{
	if (bTransactionInProgress)
	{
		FString kError = TEXT("UNREALCASCADE: Failed to begin transaction - ");
		kError += Description.ToString();
		checkf(0, TEXT("%s"), *kError);
		return false;
	}

	GEditor->Trans->Begin( NULL, Description );
	TransactionDescription = Description;
	bTransactionInProgress = true;

	return true;
}

bool FCascade::EndTransaction(const FText& Description)
{
	if (!bTransactionInProgress)
	{
		FString kError = TEXT("UNREALCASCADE: Failed to end transaction - ");
		kError += Description.ToString();
		checkf(0, TEXT("%s"), *kError);
		return false;
	}

	if ( !Description.EqualTo( TransactionDescription ) )
	{
		UE_LOGF(LogCascade, Log, "Cascade -   EndTransaction = %ls --- Curr = %ls", *Description.ToString(), *TransactionDescription.ToString());
		return false;
	}

	GEditor->Trans->End();

	TransactionDescription = FText::GetEmpty();
	bTransactionInProgress = false;

	return true;
}

void FCascade::ModifySelectedObjects()
{
	if (SelectedEmitter)
	{
		ModifyEmitter(SelectedEmitter);
	}
	if (SelectedModule)
	{
		SelectedModule->Modify();
	}
}

void FCascade::ModifyParticleSystem(bool bInModifyEmitters)
{
	ParticleSystem->Modify();
	if (bInModifyEmitters == true)
	{
		for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIdx];
			if (Emitter != NULL)
			{
				ModifyEmitter(Emitter);
			}
		}
	}
	ParticleSystemComponent->Modify();
}

void FCascade::ModifyEmitter(UParticleEmitter* Emitter)
{
	if (Emitter)
	{
		Emitter->Modify();
		for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
		{
			UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIndex];
			if (LODLevel)
			{
				LODLevel->Modify();
			}
		}
	}
}

TSharedRef<SWidget> FCascade::GenerateBoundsMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.AddMenuEntry(FCascadeCommands::Get().ToggleBounds_SetFixedBounds);
	
	return MenuBuilder.MakeWidget();
}

void FCascade::ToggleDrawOption(int32 Element)
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->ToggleDrawElement((FCascadeEdPreviewViewportClient::EDrawElements)Element);
		PreviewViewport->RefreshViewport();
	}
}

bool FCascade::IsDrawOptionEnabled(int32 Element) const
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		return PreviewViewport->GetViewportClient()->GetDrawElement((FCascadeEdPreviewViewportClient::EDrawElements)Element);
	}
	else
	{
		return false;
	}
}

void FCascade::OnViewEmitterTickTimes()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::EmitterTickTimes);
}

bool FCascade::IsViewEmitterTickTimesChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::EmitterTickTimes);
}


void FCascade::OnViewOriginAxis()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::OriginAxis);
}

bool FCascade::IsViewOriginAxisChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::OriginAxis);
}

void FCascade::OnViewParticleCounts()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::ParticleCounts);
}

bool FCascade::IsViewParticleCountsChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::ParticleCounts);
}

void FCascade::OnViewParticleEventCounts()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::ParticleEvents);
}

bool FCascade::IsViewParticleEventCountsChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::ParticleEvents);
}

void FCascade::OnViewParticleTimes()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::ParticleTimes);
}

bool FCascade::IsViewParticleTimesChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::ParticleTimes);
}

void FCascade::OnViewParticleMemory()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::ParticleMemory);
}

bool FCascade::IsViewParticleMemoryChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::ParticleMemory);
}

void FCascade::OnViewSystemCompleted()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::ParticleSystemCompleted);
}

bool FCascade::IsViewSystemCompletedChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::ParticleSystemCompleted);
}

void FCascade::OnViewGeometry()
{
	if ((PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid()) &&
		PreviewViewport->GetViewportClient()->GetFloorComponent())
	{
		UStaticMeshComponent* FloorComponent = PreviewViewport->GetViewportClient()->GetFloorComponent();
		FPreviewScene& PreviewScene = PreviewViewport->GetViewportClient()->GetPreviewScene();
		bool bIsVisible = !FloorComponent->IsVisible();

		FloorComponent->SetVisibility(bIsVisible);

		EditorOptions->bShowFloor = bIsVisible;
		EditorOptions->SaveConfig();

		PreviewScene.RemoveComponent(FloorComponent);
		PreviewScene.AddComponent(FloorComponent, FTransform::Identity);

		PreviewViewport->RefreshViewport();
	}
}

bool FCascade::IsViewGeometryChecked() const
{
	if ((PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid()) &&
		PreviewViewport->GetViewportClient()->GetFloorComponent())
	{
		return PreviewViewport->GetViewportClient()->GetFloorComponent()->IsVisible();
	}
	else
	{
		return false;
	}
}

void FCascade::OnViewGeometryProperties()
{
	TSharedPtr<SWindow> PinnedGeometryPropertiesWindow = GeometryPropertiesWindow.Pin();

	if ((PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid()) &&
		PreviewViewport->GetViewportClient()->GetFloorComponent() &&
		!PinnedGeometryPropertiesWindow.IsValid() )
	{
		UStaticMeshComponent* FloorComponent = PreviewViewport->GetViewportClient()->GetFloorComponent();
		TArray<UObject*> SelectedObjects;
		SelectedObjects.Add(FloorComponent);
		
		GeometryPropertiesWindow = ObjectTools::OpenPropertiesForSelectedObjects(SelectedObjects);
	}
	else
	{
		PinnedGeometryPropertiesWindow->BringToFront(true);
	}
}

void FCascade::OnViewLocalVectorFields()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::VectorFields);
}

bool FCascade::IsViewLocalVectorFieldsChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::VectorFields);
}

void FCascade::OnRestartSimulation()
{
	RestartParticleSystem();
}

void FCascade::OnRestartInLevel()
{
	RestartParticleSystem();

	for (TObjectIterator<UParticleSystemComponent> It;It;++It)
	{
		if (It->Template && It->Template == ParticleSystemComponent->Template)
		{
			UParticleSystemComponent* PSysComp = *It;

			if (PSysComp->IsRegistered())
			{
				// Check for a valid template
				check(PSysComp->Template);
				PSysComp->ResetParticles(true);
				PSysComp->bIsViewRelevanceDirty = true;
				PSysComp->CachedViewRelevanceFlags.Empty();
				PSysComp->Template->bShouldResetPeakCounts = true;
				PSysComp->ActivateSystem();
				PSysComp->ConditionalCacheViewRelevanceFlags();
				PSysComp->MarkRenderStateDirty();
				PSysComp->SetManagingSignificance(true);
				PSysComp->SetRequiredSignificance(RequiredSignificance);
			}
		}
	}
}

void FCascade::OnSaveThumbnailImage()
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->CreateThumbnail();
	}
}

void FCascade::OnToggleOrbitMode()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::Orbit);
}

bool FCascade::IsToggleOrbitModeChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::Orbit);
}

void FCascade::OnToggleMotion()
{
	bIsToggleMotion = !bIsToggleMotion;
}

bool FCascade::IsToggleMotionChecked() const
{
	return bIsToggleMotion;
}

void FCascade::OnSetMotionRadius()
{
	FString DefaultText = FString::Printf(TEXT("%.2f"), MotionModeRadius);
	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(NSLOCTEXT("Cascade", "MotionRadius", "Motion Radius: "))
		.DefaultText(FText::FromString( DefaultText ) )
		.OnTextCommitted(this, &FCascade::MotionRadiusCommitted)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		PreviewViewport.ToSharedRef(),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void FCascade::OnViewMode(EViewModeIndex ViewMode)
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->SetViewMode(ViewMode);

		ReassociateParticleSystem();

		PreviewViewport->RefreshViewport();
	}
}

bool FCascade::IsViewModeChecked(EViewModeIndex ViewMode) const
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		return PreviewViewport->GetViewportClient()->IsViewModeEnabled( ViewMode );
	}
	else
	{
		return false;
	}
}

void FCascade::OnToggleBounds()
{
	ToggleDrawOption(FCascadeEdPreviewViewportClient::Bounds);
}

bool FCascade::IsToggleBoundsChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::Bounds);
}

void FCascade::OnToggleBoundsSetFixedBounds()
{
	FText Transaction = NSLOCTEXT("UnrealEd", "CascadeSetFixedBounds", "Set Fixed Bounds");

	BeginTransaction( Transaction );

	// Force the component to update its bounds.
	ParticleSystemComponent->ForceUpdateBounds();

	// Grab the current bounds of the PSysComp & set it on the PSystem itself
	ParticleSystem->Modify();
	ParticleSystem->FixedRelativeBoundingBox.Min = ParticleSystemComponent->Bounds.GetBoxExtrema(0);
	ParticleSystem->FixedRelativeBoundingBox.Max = ParticleSystemComponent->Bounds.GetBoxExtrema(1);
	ParticleSystem->FixedRelativeBoundingBox.IsValid = true;
	ParticleSystem->bUseFixedRelativeBoundingBox = true;

	ParticleSystem->MarkPackageDirty();

	EndTransaction( Transaction );

	if ((SelectedModule == NULL) && (SelectedEmitter == NULL))
	{
		TArray<UObject*> NewSelection;
		NewSelection.Add(ParticleSystem);
		SetSelection(NewSelection);
	}

	ReassociateParticleSystem();
}

void FCascade::OnTogglePostProcess()
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->EngineShowFlags.PostProcessing = !PreviewViewport->GetViewportClient()->EngineShowFlags.PostProcessing;
		PreviewViewport->RefreshViewport();
	}
}

bool FCascade::IsTogglePostProcessChecked() const
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		return PreviewViewport->GetViewportClient()->EngineShowFlags.PostProcessing;
	}
	else
	{
		return false;
	}
}

void FCascade::OnToggleGrid()
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		// Toggle the grid and worldbox.
		FEditorCommonDrawHelper& DrawHelper = PreviewViewport->GetViewportClient()->GetDrawHelper();
		bool bShowGrid = !DrawHelper.bDrawGrid;
		EditorOptions->bShowGrid = bShowGrid;
		EditorOptions->SaveConfig();
		DrawHelper.bDrawGrid = bShowGrid;

		PreviewViewport->GetViewportClient()->EngineShowFlags.SetGrid(bShowGrid);
		PreviewViewport->RefreshViewport();
	}
}

bool FCascade::IsToggleGridChecked() const
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		return PreviewViewport->GetViewportClient()->GetDrawHelper().bDrawGrid;
	}
	else
	{
		return false;
	}
}

void FCascade::OnPlay()
{
	if (!FMath::IsNearlyZero(TimeScale))
	{
		CachedTimeScale = TimeScale;
		TimeScale = 0.0f;
	}
	else
	{
		TimeScale = CachedTimeScale;
	}
}

bool FCascade::IsPlayChecked() const
{
	return TimeScale > KINDA_SMALL_NUMBER;
}

void FCascade::OnAnimSpeed(float Speed)
{
	TimeScale = Speed;
}

bool FCascade::IsAnimSpeedChecked(float Speed) const
{
	if (TimeScale > KINDA_SMALL_NUMBER)
	{
		return FMath::IsNearlyEqual(TimeScale, Speed);
	}
	else
	{
		return FMath::IsNearlyEqual(CachedTimeScale, Speed);
	}
}

void FCascade::OnToggleLoopSystem()
{
	bIsToggleLoopSystem = !bIsToggleLoopSystem;

	if (!bIsToggleLoopSystem)
	{
		bIsPendingReset = false;
	}
}

bool FCascade::IsToggleLoopSystemChecked() const
{
	return bIsToggleLoopSystem;
}

void FCascade::OnToggleRealtime()
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		PreviewViewport->GetViewportClient()->ToggleRealtime();
	}
}

bool FCascade::IsToggleRealtimeChecked() const
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		return PreviewViewport->GetViewportClient()->IsRealtime();
	}
	else
	{
		return false;
	}
}

void FCascade::OnBackgroundColor()
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		if (UParticleSystem* LocalParticleSystem = GetParticleSystem())
		{
			TWeakObjectPtr<UParticleSystem> WeakParticleSystem = LocalParticleSystem;
			FColorPickerArgs PickerArgs = FColorPickerArgs(LocalParticleSystem->BackgroundColor, FOnLinearColorValueChanged::CreateLambda([WeakParticleSystem](FLinearColor NewValue)
				{
					if (UParticleSystem* PinnedParticleSystem = WeakParticleSystem.Get())
					{
						PinnedParticleSystem->BackgroundColor = NewValue.ToFColorSRGB();
					}
				}));
			PickerArgs.bClampValue = true;
			PickerArgs.ParentWidget = PreviewViewport;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));

			OpenColorPicker(PickerArgs);
		}
	}
}

void FCascade::OnToggleWireframeSphere()
{
	if (!IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::WireSphere))
	{
		FString DefaultText = FString::Printf(TEXT("%.2f"), PreviewViewport->GetViewportClient()->GetWireSphereRadius());
		TSharedRef<STextEntryPopup> TextEntry = 
			SNew(STextEntryPopup)
			.Label(NSLOCTEXT("Cascade", "SphereRadius", "Sphere Radius: "))
			.DefaultText(FText::FromString(DefaultText))
			.OnTextCommitted(this, &FCascade::SphereRadiusCommitted)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(false);

		EntryMenu = FSlateApplication::Get().PushMenu(
			PreviewViewport.ToSharedRef(),
			FWidgetPath(),
			TextEntry,
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
			);
	}
	else
	{
		ToggleDrawOption(FCascadeEdPreviewViewportClient::WireSphere);
	}
}

bool FCascade::IsToggleWireframeSphereChecked() const
{
	return IsDrawOptionEnabled(FCascadeEdPreviewViewportClient::WireSphere);
}

void FCascade::OnUndo()
{
	if (GEditor->Trans->Undo())
	{
		ForceUpdate();
		OnRestartInLevel();
	}
}

void FCascade::OnRedo()
{
	if (GEditor->Trans->Redo())
	{
		ForceUpdate();
		OnRestartInLevel();
	}
}

void FCascade::OnRegenerateLowestLODDuplicatingHighest()
{
	bool bDupeHighest = true;
	RegenerateLowestLOD(bDupeHighest);
}

void FCascade::OnRegenerateLowestLOD()
{
	bool bDupeHighest = false;
	RegenerateLowestLOD(bDupeHighest);
}

void FCascade::OnDetailMode(EDetailMode InDetailMode)
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		if (DetailMode == InDetailMode)
		{
			return;
		}
		
		// Set the detail mode values on in-level particle systems
		for (TObjectIterator<UParticleSystemComponent> It;It;++It)
		{
			if (It->Template == ParticleSystemComponent->Template)
			{
				It->EditorDetailMode = GEngine->bEnableEditorPSysRealtimeLOD ? GetCachedScalabilityCVars().DetailMode : InDetailMode;
			}
		}

		DetailMode = InDetailMode;
		
		RestartParticleSystem();
	}
}

bool FCascade::IsDetailModeChecked(EDetailMode InDetailMode) const
{
	return DetailMode == InDetailMode;
}

void FCascade::OnSignificance(EParticleSignificanceLevel InSignificance)
{
	if (PreviewViewport.IsValid() && PreviewViewport->GetViewportClient().IsValid())
	{
		if (RequiredSignificance == InSignificance)
		{
			return;
		}

		// Set the detail mode values on in-level particle systems
		for (TObjectIterator<UParticleSystemComponent> It; It; ++It)
		{
			if (It->Template == ParticleSystemComponent->Template)
			{
				It->SetManagingSignificance(true);
				It->SetRequiredSignificance(InSignificance);
			}
		}

		RequiredSignificance = InSignificance;
	}
}

bool FCascade::IsSignificanceChecked(EParticleSignificanceLevel InSignificance) const
{
	return RequiredSignificance == InSignificance;
}

void FCascade::OnJumpToLowestLOD()
{
	if (ParticleSystem->Emitters.Num() == 0)
	{
		return;
	}

	int32 Value = 0;

	SetLODValue(Value);
	SetSelectedModule(SelectedEmitter, SelectedModule);

	if (PreviewViewport.IsValid())
	{
		PreviewViewport->RefreshViewport();
	}

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnJumpToLowerLOD()
{
	if (ParticleSystem->Emitters.Num() == 0)
	{
		return;
	}

	int32 LODValue = GetCurrentlySelectedLODLevelIndex();

	// Find the next lower LOD...
	// We can use any emitter, since they will all have the same number of LOD levels
	UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
	if (Emitter)
	{
		// Go from the low to the high...
		for (int32 LODIndex = Emitter->LODLevels.Num() - 1; LODIndex >= 0; LODIndex--)
		{
			UParticleLODLevel* LODLevel	= Emitter->LODLevels[LODIndex];
			if (LODLevel)
			{
				if (LODLevel->Level < LODValue)
				{
					SetLODValue(LODLevel->Level);
					SetSelectedModule(SelectedEmitter, SelectedModule);
					
					if (PreviewViewport.IsValid())
					{
						PreviewViewport->RefreshViewport();
					}

					if (EmitterCanvas.IsValid())
					{
						EmitterCanvas->RefreshViewport();
					}

					break;
				}
			}
		}
	}
}

void FCascade::OnAddLODAfterCurrent()
{
	bool bBeforeCurrent = false;
	AddLOD(bBeforeCurrent);
}

void FCascade::OnAddLODBeforeCurrent()
{
	bool bBeforeCurrent = true;
	AddLOD(bBeforeCurrent);
}

void FCascade::OnJumpToHigherLOD()
{
	if (ParticleSystem->Emitters.Num() == 0)
	{
		return;
	}

	int32 LODValue = GetCurrentlySelectedLODLevelIndex();
	// Find the next higher LOD...
	// We can use any emitter, since they will all have the same number of LOD levels
	UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
	if (Emitter)
	{
		for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
		{
			UParticleLODLevel* LODLevel	= Emitter->LODLevels[LODIndex];
			if (LODLevel)
			{
				if (LODLevel->Level > LODValue)
				{
					SetLODValue(LODLevel->Level);
					SetSelectedModule(SelectedEmitter, SelectedModule);

					if (PreviewViewport.IsValid())
					{
						PreviewViewport->RefreshViewport();
					}

					if (EmitterCanvas.IsValid())
					{
						EmitterCanvas->RefreshViewport();
					}
					
					break;
				}
			}
		}
	}
}

void FCascade::OnJumpToHighestLOD()
{
	if (ParticleSystem->Emitters.Num() == 0)
	{
		return;
	}

	int32 Value = ParticleSystem->Emitters[0]->LODLevels.Num() - 1;

	SetLODValue(Value);
	SetSelectedModule(SelectedEmitter, SelectedModule);

	if (PreviewViewport.IsValid())
	{
		PreviewViewport->RefreshViewport();
	}

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnJumpToLODIndex(int32 LODLevel)
{
	if (ParticleSystem->Emitters.Num() == 0)
	{
		return;
	}

	int32 Value = FMath::Clamp(LODLevel, 0, ParticleSystem->Emitters[0]->LODLevels.Num() - 1);

	SetLODValue(Value);
	SetSelectedModule(SelectedEmitter, SelectedModule);

	if (PreviewViewport.IsValid())
	{
		PreviewViewport->RefreshViewport();
	}

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnDeleteLOD()
{
	UParticleEmitter* Emitter = ParticleSystem->Emitters[0];
	if (Emitter == NULL)
	{
		return;
	}

	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( NSLOCTEXT("UnrealEd", "CascadeLODDelete", "Delete LOD") ) == false)
		{
			return;
		}
	}

	int32	Selection = GetCurrentlySelectedLODLevelIndex();
	if ((Selection < 0) || ((Selection == 0) && (Emitter->LODLevels.Num() == 1)))
	{
		FNotificationInfo Info( NSLOCTEXT("UnrealEd", "CascadeCantDeleteLOD", "Can't delete - only LOD level") );
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "CascadeDeleteLOD", "Delete LOD");

	// Delete the setting...
	BeginTransaction( Transaction );
	ModifyParticleSystem(true);

	// Remove the LOD entry from the distance array
	for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel	= Emitter->LODLevels[LODIndex];
		if (LODLevel)
		{
			if ((LODLevel->Level == Selection) && (ParticleSystem->LODDistances.Num() > LODLevel->Level))
			{
				ParticleSystem->LODDistances.RemoveAt(LODLevel->Level);
				break;
			}
		}
	}

	for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel	= Emitter->LODLevels[LODIndex];
		if (LODLevel)
		{
			if ((LODLevel->Level == Selection) && (ParticleSystem->LODSettings.Num() > LODLevel->Level))
			{
				ParticleSystem->LODSettings.RemoveAt(LODLevel->Level);
				break;
			}
		}
	}

	// Remove the level from each emitter in the system
	for (int32 EmitterIndex = 0; EmitterIndex < ParticleSystem->Emitters.Num(); EmitterIndex++)
	{
		Emitter = ParticleSystem->Emitters[EmitterIndex];
		if (Emitter)
		{
			for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* LODLevel	= Emitter->LODLevels[LODIndex];
				if (LODLevel)
				{
					if (LODLevel->Level == Selection)
					{
						// Clear out the flags from the modules.
						LODLevel->RequiredModule->LODValidity &= ~(1 << LODLevel->Level);
						LODLevel->SpawnModule->LODValidity &= ~(1 << LODLevel->Level);
						if (LODLevel->TypeDataModule)
						{
							LODLevel->TypeDataModule->LODValidity &= ~(1 << LODLevel->Level);
						}

						for (int32 ModuleIndex = 0; ModuleIndex < LODLevel->Modules.Num(); ModuleIndex++)
						{
							UParticleModule* PModule = LODLevel->Modules[ModuleIndex];
							if (PModule)
							{
								PModule->LODValidity  &= ~(1 << LODLevel->Level);
							}
						}

						// Delete it and shift all down
						Emitter->LODLevels.RemoveAt(LODIndex);

						for (; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
						{
							UParticleLODLevel* RemapLODLevel	= Emitter->LODLevels[LODIndex];
							if (RemapLODLevel)
							{
								RemapLODLevel->SetLevelIndex(RemapLODLevel->Level - 1);
							}
						}
						break;
					}
				}
			}
		}
	}

	ParticleSystem->SetupSoloing();

	check(bTransactionInProgress);
	EndTransaction( Transaction );

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.DeleteLOD"), FAnalyticsEventAttribute(TEXT("Index"), Selection));
	}

	ForceUpdate();

	OnRestartInLevel();
}

void FCascade::OnDeleteModule(bool bConfirm)
{
	if (!SelectedModule || !SelectedEmitter)
	{
		return;
	}

	if (SelectedEmitter->bCollapsed == true)
	{
		// Should never get in here!
		return;
	}

	if (SelectedModuleIndex == INDEX_NONE)
	{
		return;
	}

	if ((SelectedModuleIndex == INDEX_REQUIREDMODULE) || 
		(SelectedModuleIndex == INDEX_SPAWNMODULE))
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Cascade_NoDeleteRequiredOrSpawn", "The Required and Spawn modules may not be deleted."));
		return;
	}

	int32	CurrLODSetting	= GetCurrentlySelectedLODLevelIndex();
	if (CurrLODSetting != 0)
	{
		// Don't allow deleting modules if not at highest LOD
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Cascade_ModuleDeleteLODWarning", "Attempting to delete module while not on highest LOD (0)"));
		return;
	}

	// If there are differences in the enabled states of the LOD levels for an emitter,
	// prompt the user to ensure they want to delete it...
	{	
	  UParticleLODLevel* LODLevel = SelectedEmitter->LODLevels[0];
	  UParticleModule* CheckModule;
	  bool bEnabledStateDifferent = false;
	  bool bEnabled = SelectedModule->bEnabled;
	  for (int32 LODIndex = 1; (LODIndex < SelectedEmitter->LODLevels.Num()) && !bEnabledStateDifferent; LODIndex++)
	  {
		  LODLevel = SelectedEmitter->LODLevels[LODIndex];
		  switch (SelectedModuleIndex)
		  {
		  case INDEX_TYPEDATAMODULE:
			  CheckModule = LODLevel->TypeDataModule;
			  break;
		  default:
			  CheckModule = LODLevel->Modules[SelectedModuleIndex];
			  break;
		  }
  
		  check(CheckModule);
  
		  if (LODLevel->IsModuleEditable(CheckModule))
		  {
			  bEnabledStateDifferent = true;
		  }
	  }
  
	  if ((bConfirm == true) && (bEnabledStateDifferent == true))
	  {
		  if ( EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "ModuleDeleteConfirm", "Module state is different in other LOD levels.\nAre you sure you want to delete it?")) )
		  {
			  return;
		  }
	  }
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "DeleteSelectedModule", "Delete Selected Module");

	BeginTransaction( Transaction );
	ModifySelectedObjects();
	ModifyParticleSystem();

	ParticleSystem->PreEditChange(NULL);

	// Find the module index...
	int32	DeleteModuleIndex	= -1;
	FString	ModuleName;

	UParticleLODLevel* HighLODLevel	= SelectedEmitter->GetLODLevel(0);
	check(HighLODLevel);
	for (int32 ModuleIndex = 0; ModuleIndex < HighLODLevel->Modules.Num(); ModuleIndex++)
	{
		UParticleModule* CheckModule = HighLODLevel->Modules[ModuleIndex];
		if (CheckModule == SelectedModule)
		{
			DeleteModuleIndex = ModuleIndex;
			ModuleName = CheckModule->GetClass()->GetName();
			break;
		}
	}

	if (SelectedModule->IsDisplayedInCurveEd(CurveEditor->GetEdSetup()) && !GetIsModuleShared(SelectedModule))
	{
		// Remove it from the curve editor!
		SelectedModule->RemoveModuleCurvesFromEditor(CurveEditor->GetEdSetup());
		CurveEditor->CurveChanged();
	}

	// Check all the others...
	for (int32 LODIndex = 1; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel	= SelectedEmitter->GetLODLevel(LODIndex);
		if (LODLevel)
		{
			UParticleModule* Module;

			if (DeleteModuleIndex >= 0)
			{
				Module = LODLevel->Modules[DeleteModuleIndex];
			}
			else
			{
				Module	= LODLevel->TypeDataModule;
			}

			if (Module)
			{
				Module->RemoveModuleCurvesFromEditor(CurveEditor->GetEdSetup());
				CurveEditor->CurveChanged();
			}
		}

	}
	CurveEditor->RefreshViewport();

	bool bNeedsListUpdated = false;

	for (int32 LODIndex = 0; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
	{
		UParticleLODLevel* LODLevel	= SelectedEmitter->GetLODLevel(LODIndex);

		// See if it is in this LODs level...
		UParticleModule* CheckModule;

		if (DeleteModuleIndex >= 0)
		{
			CheckModule = LODLevel->Modules[DeleteModuleIndex];
		}
		else
		{
			CheckModule = LODLevel->TypeDataModule;
		}

		if (CheckModule)
		{
			if (CheckModule->IsA(UParticleModuleTypeDataBase::StaticClass()))
			{
				check(LODLevel->TypeDataModule == CheckModule);
				LODLevel->TypeDataModule = NULL;
			}
			else if (CheckModule->IsA(UParticleModuleEventGenerator::StaticClass()))
			{
				LODLevel->EventGenerator = NULL;
			}
			LODLevel->Modules.Remove(CheckModule);
			bNeedsListUpdated = true;
		}
	}

	if (bNeedsListUpdated)
	{
		SelectedEmitter->UpdateModuleLists();
	}

	ParticleSystem->PostEditChange();

	EndTransaction( Transaction );

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.DeleteModule"), TEXT("Class"), ModuleName);
	}

	SetSelectedEmitter(SelectedEmitter);

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}

	ParticleSystem->MarkPackageDirty();
}

void FCascade::OnRefreshModule()
{
	if (SelectedModule && SelectedEmitter)
	{
		SelectedModule->RefreshModule(ParticleSystem->CurveEdSetup, SelectedEmitter, GetCurrentlySelectedLODLevelIndex());
	}
}

void FCascade::OnSyncMaterial()
{
	TArray<UObject*> Objects;

	if (SelectedModule)
	{
		UParticleModuleRequired* RequiredModule = Cast<UParticleModuleRequired>(SelectedModule);
		if (RequiredModule)
		{
			Objects.Add(RequiredModule->Material);
		}
	}

	// Sync the generic browser to the object list.
	GEditor->SyncBrowserToObjects(Objects);
}

void FCascade::OnUseMaterial()
{
	if (SelectedModule && SelectedEmitter)
	{
		UParticleModuleRequired* RequiredModule = Cast<UParticleModuleRequired>(SelectedModule);
		if (RequiredModule)
		{
			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
			UObject* Obj = GEditor->GetSelectedObjects()->GetTop(UMaterialInterface::StaticClass());
			if (Obj)
			{
				UMaterialInterface* SelectedMaterial = Cast<UMaterialInterface>(Obj);
				if (SelectedMaterial)
				{
					RequiredModule->Material = SelectedMaterial;
					SelectedEmitter->PostEditChange();
				}
			}
		}
	}
}

void FCascade::OnDupeFromHigher()
{
	bool bDoShare = false;
	bool bUseHighest = false;
	DuplicateModule(bDoShare, bUseHighest);
}

void FCascade::OnShareFromHigher()
{
	bool bDoShare = true;
	bool bUseHighest = false;
	DuplicateModule(bDoShare, bUseHighest);
}

void FCascade::OnDupeFromHighest()
{
	bool bDoShare = false;
	bool bUseHighest = true;
	DuplicateModule(bDoShare, bUseHighest);
}

void FCascade::OnSetRandomSeed()
{
	if ((SelectedModule != NULL) && (SelectedModule->SupportsRandomSeed()))
	{
		FText Transaction = NSLOCTEXT("UnrealEd", "CASC_SetRandomSeed", "Set Random Seed");

		BeginTransaction( Transaction );

		ParticleSystem->PreEditChange(NULL);
		ParticleSystemComponent->PreEditChange(NULL);

		int32 RandomSeed = FMath::RoundToInt(static_cast<float>(RAND_MAX) * RandomStream.FRand());
		if (SelectedModule->SetRandomSeedEntry(0, RandomSeed) == false)
		{
			UE_LOGF(LogCascade, Warning, "Failed to set random seed entry on module %ls", *(SelectedModule->GetClass()->GetName()));
		}

		ParticleSystemComponent->PostEditChange();
		ParticleSystem->PostEditChange();

		EndTransaction( Transaction );

		// Refresh viewport
		if (EmitterCanvas.IsValid())
		{
			EmitterCanvas->RefreshViewport();
		}
	}
}

void FCascade::OnConvertToSeeded()
{
	if ((SelectedModule != NULL) && (SelectedModule->SupportsRandomSeed() == false))
	{
		// See if there is a seeded version of this module...
		UClass* CurrentClass = SelectedModule->GetClass();
		check(CurrentClass);
		FString ClassName = CurrentClass->GetName();
		UE_LOGF(LogCascade, Log, "Non-seeded module %ls", *ClassName);
		// This only works if the seeded version is names <ClassName>_Seeded!!!!
		FString SeededClassName = ClassName + TEXT("_Seeded");
		UClass* SeededClass = UClass::TryFindTypeSlow<UClass>(SeededClassName);
		if (SeededClass != NULL)
		{
			// Find the module index
			UParticleLODLevel* BaseLODLevel = GetCurrentlySelectedLODLevel();
			if (BaseLODLevel != NULL)
			{
				check(BaseLODLevel->Level == 0);

				int32 ConvertModuleIdx = INDEX_NONE;
				for (int32 CheckModuleIdx = 0; CheckModuleIdx < BaseLODLevel->Modules.Num(); CheckModuleIdx++)
				{
					if (BaseLODLevel->Modules[CheckModuleIdx] == SelectedModule)
					{
						ConvertModuleIdx = CheckModuleIdx;
						break;
					}
				}

				check(ConvertModuleIdx != INDEX_NONE);

				FText Transaction = NSLOCTEXT("UnrealEd", "CASC_ConvertToSeeded", "Convert To Seeded");

				// We need to do this for *all* copies of this module.
				BeginTransaction( Transaction );
				if (ConvertModuleToSeeded(ParticleSystem, SelectedEmitter, ConvertModuleIdx, SeededClass, true) == false)
				{
					UE_LOGF(LogCascade, Warning, "Failed to convert module!");
				}
				EndTransaction(Transaction);

				//Have to reset all existing component using this system.
				FParticleResetContext ResetCtx;
				ResetCtx.AddTemplate(ParticleSystem);

				SetSelectedModule(SelectedEmitter, BaseLODLevel->Modules[ConvertModuleIdx]);

				if (EmitterCanvas.IsValid())
				{
					EmitterCanvas->RefreshViewport();
				}
			}
		}
	}
}

void ParticleSystem_DumpInfo(UParticleSystem* InParticleSystem)
{
#if UE_BUILD_DEBUG
	if (InParticleSystem != NULL)
	{
		UE_LOGF(LogCascade, Log, "Dumping info for %ls", *(InParticleSystem->GetPathName()));
		UE_LOGF(LogCascade, Log, "\tEmitterCount = %d", InParticleSystem->Emitters.Num());
		for (int32 EmitterIdx = 0; EmitterIdx < InParticleSystem->Emitters.Num(); EmitterIdx++)
		{
			UE_LOGF(LogCascade, Log, "\t\tEmitter %d", EmitterIdx);
			UParticleEmitter* Emitter = InParticleSystem->Emitters[EmitterIdx];
			if (Emitter != NULL)
			{
				UE_LOGF(LogCascade, Log, "\t\t\tLODLevels %d", Emitter->LODLevels.Num());
				for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
					if (LODLevel != NULL)
					{
						UE_LOGF(LogCascade, Log, "\t\t\t\tLODLevel %d", LODIdx);
						FString ModuleDump = TEXT("\t\t\t\t");

						for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							UParticleModule* Module = LODLevel->Modules[ModuleIdx];
							ModuleDump += FString::Printf(TEXT("0x%08" PTRINT_FMT ","), PTRINT(Module));
						}
						UE_LOGF(LogCascade, Log, "%ls", *ModuleDump);
					}
					else
					{
						UE_LOGF(LogCascade, Log, "\t\t\t\t*** NULL");
					}
				}
			}
			else
			{
				UE_LOGF(LogCascade, Log, "\t\t\t*** NULL");
			}
		}
	}
#endif
}

bool FCascade::ConvertModuleToSeeded(UParticleSystem* ParticleSystem, UParticleEmitter* InEmitter, int32 InModuleIdx, UClass* InSeededClass, bool bInUpdateModuleLists)
{
	ParticleSystem_DumpInfo(ParticleSystem);
	// 
	for (int32 LODIdx = 0; LODIdx < InEmitter->LODLevels.Num(); LODIdx++)
	{
		UParticleLODLevel* LODLevel = InEmitter->LODLevels[LODIdx];
		UParticleModule* ConvertModule = LODLevel->Modules[InModuleIdx];
		check(ConvertModule != NULL);

		UParticleModule* NewModule = ConvertModule;
		if ((LODIdx == 0) || ((ConvertModule->LODValidity & (1 << (LODIdx - 1))) == 0))
		{
			NewModule = CastChecked<UParticleModule>(StaticDuplicateObject(ConvertModule, ParticleSystem, NAME_None, RF_AllFlags, InSeededClass));

			// Since we used the non-randomseed module to create, this flag won't be set during construction...
			NewModule->bSupportsRandomSeed = true;

			FParticleRandomSeedInfo* RandSeedInfo = NewModule->GetRandomSeedInfo();
			if (RandSeedInfo != NULL)
			{
				RandSeedInfo->bResetSeedOnEmitterLooping = true;
				RandSeedInfo->RandomSeeds.Add(FMath::TruncToInt(RandomStream.FRand() * static_cast<float>(UINT_MAX)));
			}
		}

		// Now we have to replace all instances of the module
		LODLevel->Modify();
		LODLevel->Modules[InModuleIdx] = NewModule;
		for (int32 SubLODIdx = LODIdx + 1; SubLODIdx < InEmitter->LODLevels.Num(); SubLODIdx++)
		{
			// If the module is shared, replace it
			UParticleLODLevel* SubLODLevel = InEmitter->LODLevels[SubLODIdx];
			if (SubLODLevel != NULL)
			{
				if (SubLODLevel->Modules[InModuleIdx] == ConvertModule)
				{
					SubLODLevel->Modify();
					SubLODLevel->Modules[InModuleIdx] = NewModule;
				}
			}
		}

		// Find the module in the array
		for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* OtherEmitter = ParticleSystem->Emitters[EmitterIdx];
			if (OtherEmitter != InEmitter)
			{
				UParticleLODLevel* OtherLODLevel = OtherEmitter->LODLevels[LODIdx];
				if (OtherLODLevel != NULL)
				{
					for (int32 OtherModuleIdx = 0; OtherModuleIdx < OtherLODLevel->Modules.Num(); OtherModuleIdx++)
					{
						UParticleModule* OtherModule = OtherLODLevel->Modules[OtherModuleIdx];
						if (OtherModule == ConvertModule)
						{
							OtherLODLevel->Modify();
							OtherLODLevel->Modules[OtherModuleIdx] = NewModule;
							for (int32 OtherSubLODIdx = LODIdx + 1; OtherSubLODIdx < OtherEmitter->LODLevels.Num(); OtherSubLODIdx++)
							{
								// If the module is shared, replace it
								UParticleLODLevel* OtherSubLODLevel = OtherEmitter->LODLevels[OtherSubLODIdx];
								if (OtherSubLODLevel != NULL)
								{
									if (OtherSubLODLevel->Modules[InModuleIdx] == ConvertModule)
									{
										OtherSubLODLevel->Modify();
										OtherSubLODLevel->Modules[InModuleIdx] = NewModule;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	if (bInUpdateModuleLists == true)
	{
		ParticleSystem->UpdateAllModuleLists();
	}

	ParticleSystem_DumpInfo(ParticleSystem);

	return true;
}

bool FCascade::ConvertAllModulesToSeeded(UParticleSystem* ParticleSystem)
{
	bool bResult = true;
	for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIdx];
		if (Emitter != NULL)
		{
			UParticleLODLevel* LODLevel = Emitter->LODLevels[0];
			if (LODLevel != NULL)
			{
				for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
				{
					UParticleModule* Module = LODLevel->Modules[ModuleIdx];
					if ((Module != NULL) && (Module->SupportsRandomSeed() == false))
					{
						// See if there is a seeded version of this module...
						UClass* CurrentClass = Module->GetClass();
						check(CurrentClass);
						FString ClassName = CurrentClass->GetName();
						UE_LOGF(LogCascade, Log, "Non-seeded module %ls", *ClassName);
						// This only works if the seeded version is names <ClassName>_Seeded!!!!
						FString SeededClassName = ClassName + TEXT("_Seeded");
						UClass* SeededClass = UClass::TryFindTypeSlow<UClass>(SeededClassName);
						if (SeededClass != NULL)
						{
							TArray<FParticleCurvePair> DistCurves;
							Module->GetCurveObjects(DistCurves);
							bool bHasUniformDistributions = false;
							for (int32 DistCurveIdx = 0; DistCurveIdx < DistCurves.Num(); DistCurveIdx++)
							{
								FParticleCurvePair& Pair = DistCurves[DistCurveIdx];
								UDistributionFloatUniform* FloatUniform = Cast<UDistributionFloatUniform>(Pair.CurveObject);
								UDistributionFloatUniformCurve* FloatUniformCurve = Cast<UDistributionFloatUniformCurve>(Pair.CurveObject);
								UDistributionVectorUniform* VectorUniform = Cast<UDistributionVectorUniform>(Pair.CurveObject);
								UDistributionVectorUniformCurve* VectorUniformCurve = Cast<UDistributionVectorUniformCurve>(Pair.CurveObject);
								if (FloatUniform || FloatUniformCurve || VectorUniform || VectorUniformCurve)
								{
									bHasUniformDistributions = true;
									break;
								}
							}

							if (bHasUniformDistributions == true)
							{
								if (ConvertModuleToSeeded(ParticleSystem, Emitter, ModuleIdx, SeededClass, false) == false)
								{
									bResult = false;
								}
							}
						}
					}
				}
			}
		}
	}

	ParticleSystem->UpdateAllModuleLists();

	if( bResult )
	{
		ParticleSystem->MarkPackageDirty();
	}

	return bResult;
}

void FCascade::OnRenameEmitter()
{
	if (!SelectedEmitter)
	{
		return;
	}

	TSharedRef<STextEntryPopup> TextEntry = 
		SNew(STextEntryPopup)
		.Label(NSLOCTEXT("Cascade", "SetEmitterName", "Emitter Name: "))
		.DefaultText(FText::FromName( SelectedEmitter->GetEmitterName() ))
		.OnTextCommitted(this, &FCascade::EmitterNameCommitted)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	EntryMenu = FSlateApplication::Get().PushMenu(
		EmitterCanvas.ToSharedRef(),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void FCascade::OnDuplicateEmitter(bool bIsShared)
{
	// Make sure there is a selected emitter
	if (!SelectedEmitter)
	{
		return;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "EmitterDuplicate", "Duplicate Emitter");

	BeginTransaction( Transaction );

	ParticleSystem->PreEditChange(NULL);
	ParticleSystemComponent->PreEditChange(NULL);

	DuplicateEmitter(SelectedEmitter, ParticleSystem, bIsShared);

	ParticleSystemComponent->PostEditChange();
	ParticleSystem->PostEditChange();

	EndTransaction( Transaction );

	// Refresh viewport
	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnDeleteEmitter()
{
	if (!SelectedEmitter)
	{
		return;
	}

	check(ParticleSystem->Emitters.Contains(SelectedEmitter));

	int32	CurrLODSetting	= GetCurrentlySelectedLODLevelIndex();
	if (SelectedEmitter->IsLODLevelValid(CurrLODSetting) == false)
	{
		return;
	}

	if (SelectedEmitter->bCollapsed == true)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "EmitterDeleteCollapsed", "Can not delete a collapsed emitter.\nExpand it and then delete."));
		return;
	}

	FText Transaction = NSLOCTEXT("UnrealEd", "DeleteEmitter", "Delete Emitter");

	if (bIsSoloing == true)
	{
		if (PromptForCancellingSoloingMode( Transaction ) == false)
		{
			return;
		}
	}

	// If there are differences in the enabled states of the LOD levels for an emitter,
	// prompt the user to ensure they want to delete it...
	UParticleLODLevel* LODLevel = SelectedEmitter->LODLevels[0];
	bool bEnabledStateDifferent = false;
	bool bEnabled = LODLevel->bEnabled;
	for (int32 LODIndex = 1; (LODIndex < SelectedEmitter->LODLevels.Num()) && !bEnabledStateDifferent; LODIndex++)
	{
		LODLevel = SelectedEmitter->LODLevels[LODIndex];
		if (bEnabled != LODLevel->bEnabled)
		{
			bEnabledStateDifferent = true;
		}
		else
		{
			if (LODLevel->IsModuleEditable(LODLevel->RequiredModule))
			{
				bEnabledStateDifferent = true;
			}
			if (LODLevel->IsModuleEditable(LODLevel->SpawnModule))
			{
				bEnabledStateDifferent = true;
			}
			if (LODLevel->TypeDataModule && LODLevel->IsModuleEditable(LODLevel->TypeDataModule))
			{
				bEnabledStateDifferent = true;
			}

			for (int32 CheckModIndex = 0; CheckModIndex < LODLevel->Modules.Num(); CheckModIndex++)
			{
				if (LODLevel->IsModuleEditable(LODLevel->Modules[CheckModIndex]))
				{
					bEnabledStateDifferent = true;
				}
			}
		}
	}

	if (bEnabledStateDifferent == true)
	{
		if ( EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "EmitterDeleteConfirm", "Emitter state is different in other LOD levels.\nAre you sure you want to delete it?")) )
		{
			return;
		}
	}

	BeginTransaction( Transaction );
	ModifyParticleSystem();

	ParticleSystem->PreEditChange(NULL);

	SelectedEmitter->RemoveEmitterCurvesFromEditor(CurveEditor->GetEdSetup());
	CurveEditor->CurveChanged();

	ParticleSystem->Emitters.Remove(SelectedEmitter);

	ParticleSystem->PostEditChange();

	EmitterToThumbnailMap.Remove(SelectedEmitter);

	SetSelectedEmitter(NULL);

	ParticleSystem->SetupSoloing();

	EndTransaction( Transaction );

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.Cascade.DeleteEmitter"));
	}

	ParticleSystem->MarkPackageDirty();

	if (EmitterCanvas.IsValid())
	{
		EmitterCanvas->RefreshViewport();
	}
}

void FCascade::OnExportEmitter()
{
	ExportSelectedEmitter();
}

void FCascade::OnExportAll()
{
	if (ParticleSystem->Emitters.Num() <= 0)
	{
		// Can't export empty PSys!
		return;
	}

	UParticleEmitter* SaveSelectedEmitter = SelectedEmitter;
	// There are more LODs in the destination than the source... Add enough to cover.
	for (int32 SrcIndex = 0; SrcIndex < ParticleSystem->Emitters.Num(); SrcIndex++)
	{
		UParticleEmitter* SrcEmitter = ParticleSystem->Emitters[SrcIndex];
		if (SrcEmitter)
		{
			bool bSkipIt = true;
			for (int32 LODIndex = 0; LODIndex < SrcEmitter->LODLevels.Num(); LODIndex++)
			{
				UParticleLODLevel* LODLevel = SrcEmitter->LODLevels[LODIndex];
				if (LODLevel && LODLevel->bEnabled)
				{
					bSkipIt = false;
					break;
				}
			}

			if (!bSkipIt)
			{
				SelectedEmitter = SrcEmitter;
				ExportSelectedEmitter();
			}
		}
	}
	SelectedEmitter = SaveSelectedEmitter;
}

void FCascade::OnSelectParticleSystem()
{
	SetSelectedEmitter(NULL);
}

void FCascade::OnNewEmitterBefore()
{
	int32 PositionOffset = 0;
	AddNewEmitter(PositionOffset);
}

void FCascade::OnNewEmitterAfter()
{
	int32 PositionOffset = 1;
	AddNewEmitter(PositionOffset);
}

void FCascade::OnRemoveDuplicateModules()
{
	FText Transaction = NSLOCTEXT("UnrealEd", "RemoveDuplicateModules", "Remove Duplicate Modules");

	BeginTransaction( Transaction );
	ModifyParticleSystem(true);

	ParticleSystem->RemoveAllDuplicateModules(false, NULL);

	check(bTransactionInProgress);
	EndTransaction( Transaction );

	ParticleSystem->MarkPackageDirty();
	ForceUpdate();

	OnRestartInLevel();
}

void FCascade::CloseEntryPopup()
{
	if (EntryMenu.IsValid())
	{
		EntryMenu.Pin()->Dismiss();
	}
}

/*-----------------------------------------------------------------------------
	UCascadeParticleSystemComponent
-----------------------------------------------------------------------------*/
UCascadeParticleSystemComponent::UCascadeParticleSystemComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UCascadeParticleSystemComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	// Don't tick cascade components during the usual level tick. Cascade will tick the component as needed.
	if ( bWarmingUp )
	{
		Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	}
}


void UCascadeParticleSystemComponent::CascadeTickComponent(float DeltaTime, enum ELevelTick TickType)
{
	// Tick the particle system component when ticked from within Cascade.
	Super::TickComponent( DeltaTime, TickType, NULL );
}

const static FName CascadeParticleSystemComponentParticleLineCheckName(TEXT("ParticleLineCheck"));

bool UCascadeParticleSystemComponent::ParticleLineCheck(FHitResult& Hit, AActor* SourceActor, const FVector& End, const FVector& Start, const FVector& Extent, const FCollisionObjectQueryParams&)
{
	if (bWarmingUp == false)
	{
		if (CascadePreviewViewportPtr && CascadePreviewViewportPtr->GetFloorComponent() && CascadePreviewViewportPtr->GetFloorComponent()->IsVisibleInEditor())
		{
			Hit = FHitResult(1.f);
			return CascadePreviewViewportPtr->GetFloorComponent()->SweepComponent( Hit, Start, End, FQuat::Identity, FCollisionShape::MakeBox(Extent) );
		}
	}

	return false;
}

void UCascadeParticleSystemComponent::UpdateLODInformation()
{
	if (GetLODLevel() != EditorLODLevel)
	{
		SetLODLevel(EditorLODLevel);
	}
}

UCascadeConfiguration::UCascadeConfiguration(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/*-----------------------------------------------------------------------------
	UCascadeConfiguration
-----------------------------------------------------------------------------*/

bool UCascadeConfiguration::IsModuleTypeValid(FName TypeDataName, FName ModuleName)
{
	CacheModuleRejections();
	return !ModuleRejections.Contains(ModuleName)
		&& !TypeDataModuleRejections.FindOrAdd(TypeDataName).Contains(ModuleName);
}

void UCascadeConfiguration::CacheModuleRejections()
{
	if (ModuleRejections.Num() == 0 && TypeDataModuleRejections.Num() == 0)
	{
		TArray<UClass*> ParticleModuleClasses;
		TArray<UClass*> ParticleModuleBaseClasses;

		for(TObjectIterator<UClass> It; It; ++It)
		{
			// Find all ParticleModule classes (ignoring abstract or ParticleTrailModule classes
			if (It->IsChildOf(UParticleModule::StaticClass()))
			{
				if (!(It->HasAnyClassFlags(CLASS_Abstract)))
				{
					ParticleModuleClasses.Add(*It);
				}
				else
				{
					ParticleModuleBaseClasses.Add(*It);
				}
			}
		}

		for (int32 ModuleIndex = 0; ModuleIndex < ModuleMenu_ModuleRejections.Num(); ++ModuleIndex)
		{
			ModuleRejections.Add(FName(*ModuleMenu_ModuleRejections[ModuleIndex]));
		}

		for (int32 TypeDataIndex = 0; TypeDataIndex < ModuleMenu_TypeDataToBaseModuleRejections.Num(); ++TypeDataIndex)
		{
			FModuleMenuMapper& MenuMapper = ModuleMenu_TypeDataToBaseModuleRejections[TypeDataIndex];
			FName TypeDataName = FName(*MenuMapper.ObjName);
			TSet<FName>& Rejections = TypeDataModuleRejections.FindOrAdd(TypeDataName);
			for (int32 BaseModuleIndex = 0; BaseModuleIndex < MenuMapper.InvalidObjNames.Num(); ++BaseModuleIndex)
			{
				FName BaseClassName = *MenuMapper.InvalidObjNames[BaseModuleIndex];
				UClass* BaseClass = NULL;
				for (int32 BaseClassIndex = 0; BaseClassIndex < ParticleModuleBaseClasses.Num(); ++BaseClassIndex)
				{
					if (ParticleModuleBaseClasses[BaseClassIndex]->GetFName() == BaseClassName)
					{
						BaseClass = ParticleModuleBaseClasses[BaseClassIndex];
						break;
					}
				}
				if (BaseClass)
				{
					for (int32 ClassIndex = 0; ClassIndex < ParticleModuleClasses.Num(); ++ClassIndex)
					{
						UClass* Class = ParticleModuleClasses[ClassIndex];
						if (Class->IsChildOf(BaseClass))
						{
							Rejections.Add(Class->GetFName());
						}
					}
				}
			}
		}

		for (int32 TypeDataIndex = 0; TypeDataIndex < ModuleMenu_TypeDataToSpecificModuleRejections.Num(); ++TypeDataIndex)
		{
			FModuleMenuMapper& MenuMapper = ModuleMenu_TypeDataToSpecificModuleRejections[TypeDataIndex];
			FName TypeDataName = FName(*MenuMapper.ObjName);
			TSet<FName>& Rejections = TypeDataModuleRejections.FindOrAdd(TypeDataName);
			for (int32 ModuleIndex = 0; ModuleIndex < MenuMapper.InvalidObjNames.Num(); ++ModuleIndex)
			{
				Rejections.Add(FName(*MenuMapper.InvalidObjNames[ModuleIndex]));
			}
		}
	}
}

