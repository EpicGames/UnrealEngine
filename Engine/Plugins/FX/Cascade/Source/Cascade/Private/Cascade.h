// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EditorUndoClient.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/Platform.h"
#include "ICascade.h"
#include "IDistCurveEditor.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "ParticleHelper.h"
#include "Particles/ParticleEmitter.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FEditPropertyChain;
class FFXSystemInterface;
class FProperty;
class FReferenceCollector;
class FSpawnTabArgs;
class FUICommandList;
class SCascadeEmitterCanvas;
class SCascadePreviewViewport;
class SDockTab;
class SWidget;
class SWindow;
class UCascadeConfiguration;
class UCascadeOptions;
class UCascadeParticleSystemComponent;
class UClass;
class UMaterialInterface;
class UObject;
class UParticleLODLevel;
class UParticleModule;
class UParticleSystem;
class UParticleSystemComponent;
class UTextureRenderTarget2D;
class UVectorFieldComponent;
struct FCurveEdEntry;
struct FParticleCurvePair;
struct FPropertyChangedEvent;
struct FRandomStream;

DECLARE_LOG_CATEGORY_EXTERN(LogCascade, Log, All);

struct FParticleEmitterThumbnail
{
	FParticleEmitterThumbnail()
		: Material(nullptr)
		, Texture(nullptr)
	{}

	TWeakObjectPtr<UMaterialInterface> Material;
	/** Texture holding the material thumbnail for canvas.  Note it is the only thing we hold onto */
	TObjectPtr<UTextureRenderTarget2D> Texture;
};

typedef TMap<TWeakObjectPtr<UParticleEmitter>, FParticleEmitterThumbnail> FParticleEmitterThumbnailMap;

/*-----------------------------------------------------------------------------
   FCascade
-----------------------------------------------------------------------------*/

class FCascade : public ICascade, public FGCObject, public FTickableEditorObject, public FNotifyHook, public FCurveEdNotifyInterface, public FEditorUndoClient
{
public:
	FCascade();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& SpawnTabArgs, FName TabIdentifier);

	/** Destructor */
	virtual ~FCascade();

	/** Edits the specified ParticleSystem object */
	void InitCascade(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit);

	/** Returns the ParticleSystem asset inspected by the editor */
	UParticleSystem* GetParticleSystem() const;

	/** Returns the particle system component */
	UCascadeParticleSystemComponent* GetParticleSystemComponent() const;

	/** Returns the vector field component */
	UVectorFieldComponent* GetLocalVectorFieldComponent() const;

	/** Returns the FX system */
	FFXSystemInterface* GetFXSystem() const;

	/** Returns the currently selected emitter */
	UParticleEmitter* GetSelectedEmitter() const;

	/** Returns the currently selected module */
	UParticleModule* GetSelectedModule() const;

	/** Returns the index of the currently selected module */
	int32 GetSelectedModuleIndex();

	/** Return the index of the currently selected LOD level */
	int32 GetCurrentlySelectedLODLevelIndex() const;

	/** Return the currently selected LOD level */
	UParticleLODLevel* GetCurrentlySelectedLODLevel() const;

	/** Return the currently selected LOD level */
	UParticleLODLevel* GetCurrentlySelectedLODLevel(UParticleEmitter* InEmitter);

	/** Returns the Cascade editor options */
	UCascadeOptions* GetEditorOptions() const;

	/** Returns the Cascade editor configuration */
	UCascadeConfiguration* GetEditorConfiguration() const;

	/** Returns the curve editor */
	TSharedPtr<IDistributionCurveEditor> GetCurveEditor();

	/** Returns the preview viewport */
	TSharedPtr<SCascadePreviewViewport> GetPreviewViewport();

	/** IsSoloing accesors */
	bool GetIsSoloing() const;
	void SetIsSoloing(bool bIsSoloing);

	/** Returns the current detail mode */
	int32 GetDetailMode() const;

	/** Returns the required significance for the fx in the cascade viewport. */
	EParticleSignificanceLevel GetRequiredSignificance()const;

	/** Returns true if the module is shared */
	bool GetIsModuleShared(UParticleModule* Module);

	/** Toggle the enabled setting on the given emitter */
	void ToggleEnableOnSelectedEmitter(UParticleEmitter* InEmitter);

	/** Refreshes all viewports and controls */
	void ForceUpdate();

	/** 
	 * Adds curves belonging to the selected module to the curve editor 
	 *
	 * @param	OutCurveEntries	Adds the curves which are for this graph node
	 *
	 * @return	true, if new curves were added to the graph, otherwise they were already present
	 */
	bool AddSelectedToGraph(TArray<const FCurveEdEntry*>& OutCurveEntries);

	/** 
	 * Makes sure only the specified curves are shown in the graph, all others are hidden
	 *
	 * @param	InCurveEntries	The curves which should be visible
	 */
	void ShowDesiredCurvesOnly(const TArray<const FCurveEdEntry*>& InCurveEntries);

	/** Methods for setting the currently selected object */
	void SetSelectedEmitter(UParticleEmitter* NewSelectedEmitter, bool bIsSimpleAssignment = false);
	void SetSelectedModule(UParticleModule* NewSelectedModule);
	void SetSelectedModule(UParticleEmitter* NewSelectedEmitter, UParticleModule* NewSelectedModule);

	/** Assigns the currently selected nodes to the property control */
	void SetSelection(TArray<UObject*> SelectedObjects);

	/** Return module class info */
	TArray<UClass*>& GetParticleModuleBaseClasses();
	TArray<UClass*>& GetParticleModuleClasses();

	/** Caches the emitter/module to be copied */
	void SetCopyEmitter(UParticleEmitter* NewEmitter);
	void SetCopyModule(UParticleEmitter* NewEmitter, UParticleModule* NewModule);

	/** Returns the dragged module list */
	TArray<UParticleModule*>& GetDraggedModuleList();

	/** Move the selected emitter by MoveAmount in the array of emitters */
	void MoveSelectedEmitter(int32 MoveAmount);

	/** Inserts a module at the specified index */
	bool InsertModule(UParticleModule* Module, UParticleEmitter* TargetEmitter, int32 TargetIndex, bool bSetSelected);

	/** Copies a module to an emitter */
	void CopyModuleToEmitter(UParticleModule* pkSourceModule, UParticleEmitter* pkTargetEmitter, UParticleSystem* pkTargetSystem, int32 TargetIndex);

	/** Toolbar/menu command methods */
	void OnCustomModuleOption(int32 Idx);
	void OnNewModule(int32 Idx);
	void OnNewEmitter();
	void OnRestartInLevel();
	void OnDeleteEmitter();
	void OnDeleteModule(bool bConfirm);
	void OnJumpToHigherLOD();
	void OnJumpToLowerLOD();
	void OnUndo();
	void OnRedo();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	
	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Rendering/ParticleSystems"));
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	
	/** FTickableEditorObject interface */
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;



	/** FNotifyHook interface */
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	virtual void NotifyPreChange(FEditPropertyChain* PropertyChain) override;
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyChain) override;

	/** FCurveEdNotifyInterface */
	virtual void PreEditCurve(TArray<UObject*> CurvesAboutToChange) override;
	virtual void PostEditCurve() override;
	virtual void MovedKey() override;
	virtual void DesireUndo() override;
	virtual void DesireRedo() override;

	/**
	 *	Convert all the modules in this particle system to their random seed variant if available
	 *
	 *	@return	bool			true if successful, false if not
	 */
	static bool ConvertAllModulesToSeeded(UParticleSystem* ParticleSystem);

	static void OnComponentActivationChange(UParticleSystemComponent* PSC, bool bActivated);

	FParticleEmitterThumbnailMap& GetEmitterToThumbnailMap() { return EmitterToThumbnailMap; }

private:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	/** Generates menu for the AnimSpeed sub menu */
	static TSharedRef<SWidget> GenerateAnimSpeedMenuContent(TSharedRef<FUICommandList> InCommandList);
	
	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Builds the toolbar widget for the ParticleSystem editor */
	void ExtendToolbar();
	
	/**	Binds our UI commands to delegates */
	void BindCommands();

	/**
	 *	Convert the given module to its random seed variant.
	 *
	 *	@param	InEmitter				The emitter to convert
	 *	@param	InModuleIdx				The index of the module to convert
	 *	@param	InSeededClass			The seeded variant class
	 *	@param	bInUpdateModuleLists	If true, update the module lists after the conversion
	 *
	 *	@return	bool					true if successful, false if not
	 */
	static bool ConvertModuleToSeeded(UParticleSystem* ParticleSystem, UParticleEmitter* InEmitter, int32 InModuleIdx, UClass* InSeededClass, bool bInUpdateModuleLists);

	/** Initializes the lists of module classes */
	void InitParticleModuleClasses();

	/** Delegate callbacks for the LOD toolbar spinbox */
	TOptional<int32> GetMaxLOD() const;
	TOptional<int32> GetCurrentLOD() const;
	void OnCurrentLODChanged(int32 NewLOD);

	/** Callback for committing a motion radius */
	void MotionRadiusCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);

	/** Callback for committing a sphere radius */
	void SphereRadiusCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);

	/** Callback for renaming an emitter */
	void EmitterNameCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);

	/** Sets the LOD level to the currently selected LOD index */
	void UpdateLODLevel();

	/** Changes the LOD level */
	void SetLODValue(int32 LODSetting);

	/** Removes and then Readds the particle system to the preview scene */
	void ReassociateParticleSystem() const;

	/** Restarts the particle system */
	void RestartParticleSystem();

	/** Prompt the user for canceling soloing mode to perform the selected operation */
	bool PromptForCancellingSoloingMode( const FText& InOperationDesc);

	/** Duplicates an existing emitter */
	bool DuplicateEmitter(UParticleEmitter* SourceEmitter, UParticleSystem* DestSystem, bool bShare);

	/** Adds a new emitter */
	void AddNewEmitter(int32 PositionOffset);

	/** Duplicates an existing module */
	void DuplicateModule(bool bDoShare, bool bUseHighest);
	
	/** Export the selected emitter for importing into another particle system */
	void ExportSelectedEmitter();

	/** Regenerates the lowest LOD */
	void RegenerateLowestLOD(bool bDupeHighest);

	/** Adds a new LOD */
	void AddLOD(bool bBeforeCurrent);

	/** Sets the selection in the curve editor */
	void SetSelectedInCurveEditor();

	/** Undo/Redo support */
	bool BeginTransaction(const FText& Description);
	bool EndTransaction(const FText& Description);
	
	/** Modify methods */
	void ModifySelectedObjects();
	void ModifyParticleSystem(bool bInModifyEmitters = false);
	void ModifyEmitter(UParticleEmitter* Emitter);

	/** Generates menus for the Bounds toolbar button */
	static TSharedRef<SWidget> GenerateBoundsMenuContent(TSharedRef<FUICommandList> InCommandList);

	/** View option helper methods */
	void ToggleDrawOption(int32 Element);
	bool IsDrawOptionEnabled(int32 Element) const;
	
	/** Toolbar/menu command methods */
	void OnViewOriginAxis();
	bool IsViewOriginAxisChecked() const;
	void OnViewParticleCounts();
	bool IsViewParticleCountsChecked() const;
	void OnViewParticleEventCounts();
	bool IsViewParticleEventCountsChecked() const;
	void OnViewParticleTimes();
	bool IsViewParticleTimesChecked() const;
	void OnViewParticleMemory();
	bool IsViewParticleMemoryChecked() const;
	void OnViewSystemCompleted();
	bool IsViewSystemCompletedChecked() const;
	void OnViewEmitterTickTimes();
	bool IsViewEmitterTickTimesChecked() const;
	void OnViewGeometry();
	bool IsViewGeometryChecked() const;
	void OnViewGeometryProperties();
	bool IsViewGeometryPropertiesChecked() const;
	void OnViewLocalVectorFields();
	bool IsViewLocalVectorFieldsChecked() const;
	void OnRestartSimulation();
	void OnSaveThumbnailImage();
	void OnToggleOrbitMode();
	bool IsToggleOrbitModeChecked() const;
	void OnToggleMotion();
	bool IsToggleMotionChecked() const;
	void OnSetMotionRadius();
	void OnViewMode(EViewModeIndex ViewMode);
	bool IsViewModeChecked(EViewModeIndex ViewMode) const;
	void OnToggleBounds();
	bool IsToggleBoundsChecked() const;
	void OnToggleBoundsSetFixedBounds();
	void OnTogglePostProcess();
	bool IsTogglePostProcessChecked() const;
	void OnToggleGrid();
	bool IsToggleGridChecked() const;
	void OnPlay();
	bool IsPlayChecked() const;
	void OnAnimSpeed(float Speed);
	bool IsAnimSpeedChecked(float Speed) const;
	void OnToggleLoopSystem();
	bool IsToggleLoopSystemChecked() const;
	void OnToggleRealtime();
	bool IsToggleRealtimeChecked() const;
	void OnBackgroundColor();
	void OnToggleWireframeSphere();
	bool IsToggleWireframeSphereChecked() const;
	void OnRegenerateLowestLODDuplicatingHighest();
	void OnRegenerateLowestLOD();
	void OnDetailMode(EDetailMode DetailMode);
	bool IsDetailModeChecked(EDetailMode DetailMode) const;
	void OnSignificance(EParticleSignificanceLevel InSignificance);
	bool IsSignificanceChecked(EParticleSignificanceLevel InSignificance) const;
	void OnJumpToHighestLOD();
	void OnAddLODAfterCurrent();
	void OnAddLODBeforeCurrent();
	void OnJumpToLowestLOD();
	void OnDeleteLOD();
	void OnJumpToLODIndex(int32 LODLevel);
	void OnRefreshModule();
	void OnSyncMaterial();
	void OnUseMaterial();
	void OnDupeFromHigher();
	void OnShareFromHigher();
	void OnDupeFromHighest();
	void OnSetRandomSeed();
	void OnConvertToSeeded();
	void OnRenameEmitter();
	void OnDuplicateEmitter(bool bIsShared);
	void OnExportEmitter();
	void OnExportAll();
	void OnSelectParticleSystem();
	void OnNewEmitterBefore();
	void OnNewEmitterAfter();
	void OnRemoveDuplicateModules();

	void CloseEntryPopup();

private:
	/** The ParticleSystem asset being inspected */
	TObjectPtr<UParticleSystem> ParticleSystem;

	/** Preview Viewport */
	TSharedPtr<SCascadePreviewViewport> PreviewViewport;

	/** Emitter Canvas */
	TSharedPtr<SCascadeEmitterCanvas> EmitterCanvas;

	/** Properties Tab */
	TSharedPtr<class IDetailsView> Details;

	/** Curve Editor */
	TSharedPtr<IDistributionCurveEditor> CurveEditor;

	/** Reference to owner of the current popup */
	TWeakPtr<class IMenu> EntryMenu;

	/** Components used for drawing the particle system in the preview viewport */
	TObjectPtr<UCascadeParticleSystemComponent> ParticleSystemComponent;
	TObjectPtr<UVectorFieldComponent> LocalVectorFieldPreviewComponent;

	/** Currently selected LOD index */
	int32 CurrentLODIdx;

	/** Config options */
	TObjectPtr<UCascadeOptions> EditorOptions;
	TObjectPtr<UCascadeConfiguration> EditorConfig;

	/** Undo/redo support */
	bool bTransactionInProgress;
	FText TransactionDescription;

	/** Selection info */
	int32 SelectedModuleIndex;
	TObjectPtr<UParticleModule> SelectedModule;
	TObjectPtr<UParticleEmitter> SelectedEmitter;
	
	/** True if an emitter is "soloing" */
	bool bIsSoloing;

	/** Cached copy info */
	TObjectPtr<UParticleModule> CopyModule;
	TObjectPtr<UParticleEmitter> CopyEmitter;

	/** View/draw info  */
	bool bIsToggleMotion;
	float MotionModeRadius;
	float AccumulatedMotionTime;
	float TimeScale;
	float CachedTimeScale;
	bool bIsToggleLoopSystem;
	bool bIsPendingReset;
	double TotalTime;
	double ResetTime;
	float ParticleMemoryUpdateTime;
	
	/** Cascade specific detail mode */
	int32 DetailMode;

	/** Required significance for cascade. */
	EParticleSignificanceLevel RequiredSignificance;

	/** Used to track changes in the global detail mode setting */
	int32 GlobalDetailMode;

	/** List of all ParticleModule subclasses */
	TArray<UClass*> ParticleModuleBaseClasses;
	TArray<UClass*> ParticleModuleClasses;
	bool bParticleModuleClassesInitialized;

	/** List of modules currently being dragged */
	TArray<UParticleModule*> DraggedModuleList;

	/** Used to enforce that all LODLevels in an emitter are either SubUV or not... */
	EParticleSubUVInterpMethod PreviousInterpolationMethod;

	/** For handling curves/distribution data */
	TObjectPtr<UObject> CurveToReplace;
	TArray<FParticleCurvePair> DynParamCurves;

	/** The geometry properties window, if it exists */
	TWeakPtr<SWindow> GeometryPropertiesWindow;

	/** Maps particle emitters to rendered thumbnails for the emitter UI */
	FParticleEmitterThumbnailMap EmitterToThumbnailMap;

	/** Random stream used to seed particle modules */
	static FRandomStream RandomStream;
};
