// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadeActions.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"
#include "Misc/CoreMiscDefines.h"

#define LOCTEXT_NAMESPACE "CascadeCommands"

/** UI_COMMAND takes long for the compile to optimize */
UE_DISABLE_OPTIMIZATION_SHIP
void FCascadeCommands::RegisterCommands()
{
	UI_COMMAND(RestartSimulation, "Restart Sim", "Restart Simulation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RestartInLevel, "Restart Level", "Restart in Level", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
	UI_COMMAND(SaveThumbnailImage, "Thumbnail", "Generate Thumbnail", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CascadePlay, "Play/Pause", "Play/Pause Simulation", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AnimSpeed_100, "100%", "100% Speed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AnimSpeed_50, "50%", "50% Speed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AnimSpeed_25, "25%", "25% Speed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AnimSpeed_10, "10%", "10% Speed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AnimSpeed_1, "1%", "1% Speed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(RegenerateLowestLODDuplicatingHighest, "Regen LOD", "Regenerate Lowest LOD, Duplicating Highest", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RegenerateLowestLOD, "Regen LOD", "Regenerate Lowest LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToHighestLOD, "Highest LOD", "Select Highest LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToHigherLOD, "Higher LOD", "Select Higher LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddLODAfterCurrent, "Add LOD", "Add LOD After Current", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddLODBeforeCurrent, "Add LOD", "Add LOD Before Current", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToLowerLOD, "Lower LOD", "Select Lower LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToLowestLOD, "Lowest LOD", "Select Lowest LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteLOD, "Delete LOD", "Delete Selected LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToLOD0, "Jump To LOD0", "LOD0", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::M));
	UI_COMMAND(JumpToLOD1, "Jump To LOD1", "LOD1", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Comma));
	UI_COMMAND(JumpToLOD2, "Jump To LOD2", "LOD2", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Period));
	UI_COMMAND(JumpToLOD3, "Jump To LOD3", "LOD3", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Slash));

	UI_COMMAND(ToggleOriginAxis, "Origin Axis", "Display Origin Axis", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_ParticleCounts, "Particle Counts", "Display Particle Counts", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_ParticleEventCounts, "Particle Event Counts", "Display Particle Event Counts", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_ParticleTimes, "Particle Times", "Display Particle Times", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_ParticleMemory, "Particle Memory", "Display Particle Memory", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_SystemCompleted, "System Completed", "Display 'Completed' when the particle system has finished playing and has not been reset (only for non-looping systems)", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(View_EmitterTickTimes, "Emitter Tick Times", "Show the tick duration for each emitter", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewMode_Wireframe, "Wireframe", "Select Wireframe Render Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewMode_Unlit, "Unlit", "Select Unlit Render Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewMode_Lit, "Lit", "Select Lit Render Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewMode_ShaderComplexity, "Shader Complexity", "Select Shader Complexity Render Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DetailMode_Low, "Low", "Select Low Detail Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DetailMode_Medium, "Medium", "Select Medium Detail Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DetailMode_High, "High", "Select High Detail Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DetailMode_Epic, "Epic", "Select Epic Detail Mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(Significance_Critical, "Critical", "Require >= Critical Significance", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(Significance_High, "High", "Require High >= Significance", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(Significance_Medium, "Medium", "Require >= Medium Significance", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(Significance_Low, "Low", "Require >= Low Significance", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ToggleGeometry, "Geometry", "Display preview mesh", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleGeometry_Properties, "Geometry Properties", "Display Preview Mesh Properties", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleLocalVectorFields, "Vector Fields", "Display Local Vector Fields", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleOrbitMode, "Orbit Mode", "Toggle Orbit Mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleMotion, "Motion", "Toggle Motion", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetMotionRadius, "Motion Radius", "Set Motion Radius", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleBounds, "Bounds", "Display Bounds", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleBounds_SetFixedBounds, "Set Fixed Bounds", "Set Fixed Bounds", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TogglePostProcess, "Post Process", "Toggle Post Process", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleGrid, "Grid", "Display Grid", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleLoopSystem, "Loop", "Toggle Looping", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleRealtime, "Realtime", "Toggles real time rendering in this viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CascadeBackgroundColor, "Background Color", "Change Background Color", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleWireframeSphere, "Wireframe Sphere", "Display Wireframe Sphere", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(DeleteModule, "Delete Module", "Delete Module", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RefreshModule, "Refresh Module", "Refresh Module", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SyncMaterial, "Sync Material", "Sync Material", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UseMaterial, "Use Material", "Use Material", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DupeFromHigher, "Duplicate From Higher", "Duplicate From Higher LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShareFromHigher, "Share From Higher", "Share From Higher LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DupeFromHighest, "Duplicate From Highest", "Duplicate From Second Highest LOD", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetRandomSeed, "Set Random Seed", "Set Random Seed", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToSeeded, "Convert To Seeded", "Convert To Seeded Module", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameEmitter, "Rename Emitter", "Rename Emitter", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateEmitter, "Duplicate Emitter", "Duplicate Emitter", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateShareEmitter, "Duplicate and Share Emitter", "Duplicate and Share Emitter", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteEmitter, "Delete Emitter", "Delete Emitter", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportEmitter, "Export Emitter", "Export Emitter", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportAllEmitters, "Export All", "Export All Emitters", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectParticleSystem, "Select Particle System", "Select Particle System", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NewEmitterBefore, "Add New Emitter Before", "Add New Emitter Before Selected", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NewEmitterAfter, "Add New Emitter After", "Add New Emitter After Selected", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveDuplicateModules, "Remove Duplicate Modules", "Remove Duplicate Modules", EUserInterfaceActionType::Button, FInputChord());
}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
