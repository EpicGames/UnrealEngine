// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/*-----------------------------------------------------------------------------
   FCascadeCommands
-----------------------------------------------------------------------------*/

class FCascadeCommands : public TCommands<FCascadeCommands>
{
public:
	/** Constructor */
	FCascadeCommands() 
		: TCommands<FCascadeCommands>("Cascade", NSLOCTEXT("Contexts", "Cascade", "Cascade"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** See tooltips in cpp for documentation */
	
	TSharedPtr<FUICommandInfo> ToggleOriginAxis;
	TSharedPtr<FUICommandInfo> View_ParticleCounts;
	TSharedPtr<FUICommandInfo> View_ParticleEventCounts;
	TSharedPtr<FUICommandInfo> View_ParticleTimes;
	TSharedPtr<FUICommandInfo> View_ParticleMemory;
	TSharedPtr<FUICommandInfo> View_SystemCompleted;
	TSharedPtr<FUICommandInfo> View_EmitterTickTimes;
	TSharedPtr<FUICommandInfo> ToggleGeometry;
	TSharedPtr<FUICommandInfo> ToggleGeometry_Properties;
	TSharedPtr<FUICommandInfo> ToggleLocalVectorFields;
	TSharedPtr<FUICommandInfo> RestartSimulation;
	TSharedPtr<FUICommandInfo> RestartInLevel;
	TSharedPtr<FUICommandInfo> SaveThumbnailImage;
	TSharedPtr<FUICommandInfo> ToggleOrbitMode;
	TSharedPtr<FUICommandInfo> ToggleMotion;
	TSharedPtr<FUICommandInfo> SetMotionRadius;
	TSharedPtr<FUICommandInfo> ViewMode_Wireframe;
	TSharedPtr<FUICommandInfo> ViewMode_Unlit;
	TSharedPtr<FUICommandInfo> ViewMode_Lit;
	TSharedPtr<FUICommandInfo> ViewMode_ShaderComplexity;
	TSharedPtr<FUICommandInfo> ToggleBounds;
	TSharedPtr<FUICommandInfo> ToggleBounds_SetFixedBounds;
	TSharedPtr<FUICommandInfo> TogglePostProcess;
	TSharedPtr<FUICommandInfo> ToggleGrid;
	TSharedPtr<FUICommandInfo> CascadePlay;
	TSharedPtr<FUICommandInfo> AnimSpeed_100;
	TSharedPtr<FUICommandInfo> AnimSpeed_50;
	TSharedPtr<FUICommandInfo> AnimSpeed_25;
	TSharedPtr<FUICommandInfo> AnimSpeed_10;
	TSharedPtr<FUICommandInfo> AnimSpeed_1;
	TSharedPtr<FUICommandInfo> ToggleLoopSystem;
	TSharedPtr<FUICommandInfo> ToggleRealtime;
	TSharedPtr<FUICommandInfo> CascadeBackgroundColor;
	TSharedPtr<FUICommandInfo> ToggleWireframeSphere;
	TSharedPtr<FUICommandInfo> RegenerateLowestLODDuplicatingHighest;
	TSharedPtr<FUICommandInfo> RegenerateLowestLOD;
	TSharedPtr<FUICommandInfo> DetailMode_Low;
	TSharedPtr<FUICommandInfo> DetailMode_Medium;
	TSharedPtr<FUICommandInfo> DetailMode_High;
	TSharedPtr<FUICommandInfo> DetailMode_Epic;
	TSharedPtr<FUICommandInfo> Significance_Critical;
	TSharedPtr<FUICommandInfo> Significance_High;
	TSharedPtr<FUICommandInfo> Significance_Medium;
	TSharedPtr<FUICommandInfo> Significance_Low;
	TSharedPtr<FUICommandInfo> JumpToHighestLOD;
	TSharedPtr<FUICommandInfo> JumpToHigherLOD;
	TSharedPtr<FUICommandInfo> AddLODAfterCurrent;
	TSharedPtr<FUICommandInfo> AddLODBeforeCurrent;
	TSharedPtr<FUICommandInfo> JumpToLowerLOD;
	TSharedPtr<FUICommandInfo> JumpToLowestLOD;
	TSharedPtr<FUICommandInfo> JumpToLOD0;
	TSharedPtr<FUICommandInfo> JumpToLOD1;
	TSharedPtr<FUICommandInfo> JumpToLOD2;
	TSharedPtr<FUICommandInfo> JumpToLOD3;
	TSharedPtr<FUICommandInfo> DeleteLOD;
	TSharedPtr<FUICommandInfo> DeleteModule;
	TSharedPtr<FUICommandInfo> RefreshModule;
	TSharedPtr<FUICommandInfo> SyncMaterial;
	TSharedPtr<FUICommandInfo> UseMaterial;
	TSharedPtr<FUICommandInfo> DupeFromHigher;
	TSharedPtr<FUICommandInfo> ShareFromHigher;
	TSharedPtr<FUICommandInfo> DupeFromHighest;
	TSharedPtr<FUICommandInfo> SetRandomSeed;
	TSharedPtr<FUICommandInfo> ConvertToSeeded;
	TSharedPtr<FUICommandInfo> RenameEmitter;
	TSharedPtr<FUICommandInfo> DuplicateEmitter;
	TSharedPtr<FUICommandInfo> DuplicateShareEmitter;
	TSharedPtr<FUICommandInfo> DeleteEmitter;
	TSharedPtr<FUICommandInfo> ExportEmitter;
	TSharedPtr<FUICommandInfo> ExportAllEmitters;
	TSharedPtr<FUICommandInfo> SelectParticleSystem;
	TSharedPtr<FUICommandInfo> NewEmitterBefore;
	TSharedPtr<FUICommandInfo> NewEmitterAfter;
	TSharedPtr<FUICommandInfo> RemoveDuplicateModules;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
