// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadeEmitterCanvasClient.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "CascadeConfiguration.h"
#include "Materials/MaterialInstance.h"
#include "Particles/ParticleSystem.h"
#include "Cascade.h"
#include "SCascadeEmitterCanvas.h"
#include "Misc/MessageDialog.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Preferences/CascadeOptions.h"
#include "Editor/UnrealEdEngine.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UnrealEdGlobals.h"
#include "CascadeActions.h"
#include "CascadeEmitterHitProxies.h"
#include "Slate/SceneViewport.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleSpriteEmitter.h"
#include "Particles/ParticleModuleRequired.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Colors/SColorPicker.h"
#include "CanvasTypes.h"
#include "Engine/Font.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"


FCascadeEmitterCanvasClient::FCascadeEmitterCanvasClient(TSharedPtr<FCascade> InCascade, TWeakPtr<SCascadeEmitterCanvas> InCascadeViewport)
	: FEditorViewportClient(nullptr)
	, CascadePtr(InCascade)
	, CascadeViewportPtr(InCascadeViewport)
	, EmitterWidth(180)
	, EmitterCollapsedWidth(18)
	, EmitterHeadHeight(60)
	, EmitterThumbBorder(5)
	, ModuleHeight(40)
	, EmptyBackgroundColor(112, 112, 112)
	, EmitterBackgroundColor(130, 130, 130)
	, EmitterSelectedColor(255, 130, 30)
	, EmitterUnselectedColor(180, 180, 180)
	, RenderModeSelected(255,200,0)
	, RenderModeUnselected(112, 112, 112)
	, Module3DDrawModeEnabledColor(255, 200, 0)
	, Module3DDrawModeDisabledColor(112,112,112)
	, RequiredModuleOffset(1)
	, SpawnModuleOffset(2)
	, ModulesOffset(3)
	, bInitializedModuleEntries(false)
{
	check(CascadePtr.IsValid() && CascadeViewportPtr.IsValid());

	UCascadeOptions* EditorOptions = CascadePtr.Pin()->GetEditorOptions();

	if (EditorOptions->bUseSlimCascadeDraw == true)
	{
		ModuleHeight = FMath::Max<int32>(EditorOptions->SlimCascadeDrawHeight, 20);
	}
	else
	{
		EditorOptions->bCenterCascadeModuleText = false;
	}

	// This window will be 2D/canvas only, so set the viewport type to None
	ViewportType = LVT_None;

	CanvasDimensions = FIntPoint(0,0);

	CurrentMoveMode = MoveMode_None;
	MouseHoldOffset = FIntPoint(0,0);
	MousePressPosition = FIntPoint(0,0);
	bMouseDragging = false;
	bMouseDown = false;
#if defined(_CASCADE_ENABLE_MODULE_DUMP_)
	bDrawDraggedModule = EditorOptions->bShowModuleDump;
#else	//#if defined(_CASCADE_ENABLE_MODULE_DUMP_)
	bDrawDraggedModule = false;
#endif	//#if defined(_CASCADE_ENABLE_MODULE_DUMP_)

	DraggedModule = NULL;

	Origin2D = FIntPoint(0,0);

	ResetDragModIndex = INDEX_NONE;

	EmptyBackgroundColor = EditorOptions->Empty_Background;
	EmptyBackgroundColor.A = 255;
	EmitterBackgroundColor = EditorOptions->Emitter_Background;
	EmitterBackgroundColor.A = 255;
	EmitterSelectedColor = EditorOptions->Emitter_Unselected;
	EmitterSelectedColor.A = 255;
	EmitterUnselectedColor = EditorOptions->Emitter_Selected;
	EmitterUnselectedColor.A = 255;

	FColor ColorOptionsUnselected[EPMT_MAX] =	{	EditorOptions->ModuleColor_General_Unselected, 
													EditorOptions->ModuleColor_TypeData_Unselected,
													EditorOptions->ModuleColor_Beam_Unselected,
													EditorOptions->ModuleColor_Trail_Unselected,
													EditorOptions->ModuleColor_Spawn_Unselected,
													EditorOptions->ModuleColor_Required_Unselected,
													EditorOptions->ModuleColor_Event_Unselected,
													EditorOptions->ModuleColor_Light_Unselected,
													EditorOptions->ModuleColor_SubUV_Unselected
												};

	FColor ColorOptionsSelected[EPMT_MAX] =	{	EditorOptions->ModuleColor_General_Selected, 
												EditorOptions->ModuleColor_TypeData_Selected,
												EditorOptions->ModuleColor_Beam_Selected,
												EditorOptions->ModuleColor_Trail_Selected,
												EditorOptions->ModuleColor_Spawn_Selected,
												EditorOptions->ModuleColor_Required_Selected,
												EditorOptions->ModuleColor_Event_Selected,
												EditorOptions->ModuleColor_Light_Selected,
												EditorOptions->ModuleColor_SubUV_Selected
											};

	for (int32 i = 0; i < EPMT_MAX; ++i)
	{
		ModuleColors[i][Selection_Unselected] = ColorOptionsUnselected[i];
		ModuleColors[i][Selection_Unselected].A = 255;
		ModuleColors[i][Selection_Selected] = ColorOptionsSelected[i];
		ModuleColors[i][Selection_Selected].A = 255;
	}

	IconTex[Icon_RenderNormal] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Normal.CASC_Normal"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderNormal]);
	IconTex[Icon_RenderCross] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Cross.CASC_Cross"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderCross]);
	IconTex[Icon_RenderPoint] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Point.CASC_Point"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderPoint]);
	IconTex[Icon_RenderNone] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_None.CASC_None"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderNone]);
	IconTex[Icon_RenderLights] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Lights.CASC_Lights"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderLights]);
	IconTex[Icon_CurveEdit] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_CurveEd.CASC_CurveEd"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_RenderLights]);
	IconTex[Icon_3DDrawEnabled] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_ModuleEnable.CASC_ModuleEnable"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_3DDrawEnabled]);
	IconTex[Icon_3DDrawDisabled] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_ModuleDisable.CASC_ModuleDisable"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_3DDrawDisabled]);
	IconTex[Icon_ModuleEnabled] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_ModuleEnable.CASC_ModuleEnable"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_ModuleEnabled]);
	IconTex[Icon_ModuleDisabled] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_ModuleDisable.CASC_ModuleDisable"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_ModuleDisabled]);
	IconTex[Icon_SoloEnabled]	 = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Solo_On.CASC_Solo_On"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_SoloEnabled]);
	IconTex[Icon_SoloDisabled] = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_Solo_Off.CASC_Solo_Off"), NULL, LOAD_None, NULL);
	check(IconTex[Icon_SoloDisabled]);

	TexModuleDisabledBackground	= (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Cascade/CASC_DisabledModule.CASC_DisabledModule"), NULL, LOAD_None, NULL);
	check(TexModuleDisabledBackground);
}

FCascadeEmitterCanvasClient::~FCascadeEmitterCanvasClient()
{
}

void FCascadeEmitterCanvasClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	if (!CascadePtr.IsValid())
	{
		return;
	}

	UpdateScrollBars();

	FVector2D ScrollBarPos = GetViewportScrollBarPositions();
	
	Origin2D.X = -ScrollBarPos.X;
	Origin2D.Y = -ScrollBarPos.Y;

	NumRejectedModulesDrawn = 0;
	ModuleErrorStrings.Reset();

	Canvas->PushAbsoluteTransform(FTranslationMatrix(FVector(Origin2D.X,Origin2D.Y,0)));

	// Clear the background to gray and set the 2D draw origin for the viewport
	if (Canvas->IsHitTesting() == false)
	{
		Canvas->Clear( EmptyBackgroundColor);
	}
	else
	{
		Canvas->Clear(FLinearColor(1.0f,1.0f,1.0f,1.0f));
	}

	int32 ViewX = InViewport->GetSizeXY().X;
	int32 ViewY = InViewport->GetSizeXY().Y;

	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();

	int32 EmitterOffset = 0;
	for(int32 i=0; i<ParticleSystem->Emitters.Num(); i++)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[i];
		if (Emitter)
		{
			DrawEmitter(i, EmitterOffset, Emitter, InViewport, Canvas);
		}
		// Move X position on to next emitter.
		if (Emitter && Emitter->bCollapsed)
		{
			EmitterOffset += EmitterCollapsedWidth;
		}
		else
		{
			EmitterOffset += EmitterWidth;
		}
		// Draw vertical line after last column
		Canvas->DrawTile(EmitterOffset - 1, 0, 1, ViewY - Origin2D.Y, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);
	}

	// Draw line under emitter headers
	Canvas->DrawTile(0, EmitterHeadHeight-1, ViewX - Origin2D.X, 1, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);

	// Draw the module dump, if it is enabled
	if (bDrawDraggedModule)
	{
		DrawModuleDump(InViewport, Canvas);
	}

	// When dragging a module.
	if ((CurrentMoveMode != MoveMode_None) && bMouseDragging)
	{
		if (DraggedModule)
		{
			DrawDraggedModule(DraggedModule, InViewport, Canvas);
		}
	}

	Canvas->PopTransform();

	// Draw module errors and warnings.
	{
		UFont* ErrorFont = GEngine->GetSmallFont();
		const int32 LineHeight = FMath::TruncToInt( ErrorFont->GetMaxCharHeight() );
		int32 DrawY = ViewY - 2 - LineHeight;

		for (int32 i = 0; i < ModuleErrorStrings.Num(); ++i)
		{
			Canvas->DrawShadowedString(
				2,
				DrawY,
				*ModuleErrorStrings[i],
				ErrorFont,
				FLinearColor::Red
				);
			DrawY -= LineHeight;
		}

		if (NumRejectedModulesDrawn != 0)
		{
			Canvas->DrawShadowedText(
				2,
				DrawY,
				NSLOCTEXT("UnrealEd", "InvalidModules", "An emitter has modules that are incompatible with its type data."),
				ErrorFont,
				FLinearColor::Red
				);
		}
	}
}

bool FCascadeEmitterCanvasClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = false;

	bool bLODIsValid = true;
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	bool bCtrlDown = EventArgs.Viewport->KeyState(EKeys::LeftControl) || EventArgs.Viewport->KeyState(EKeys::RightControl);
	bool bShiftDown = EventArgs.Viewport->KeyState(EKeys::LeftShift) || EventArgs.Viewport->KeyState(EKeys::RightShift);
	bool bAltDown = EventArgs.Viewport->KeyState(EKeys::LeftAlt) || EventArgs.Viewport->KeyState(EKeys::RightAlt);
	int32 HitX = EventArgs.Viewport->GetMouseX();
	int32 HitY = EventArgs.Viewport->GetMouseY();
	FIntPoint MousePos = FIntPoint(HitX, HitY);

	if (EventArgs.Key == EKeys::LeftMouseButton || EventArgs.Key == EKeys::RightMouseButton)
	{
		bHandled = true;
		
		if (EventArgs.Event == IE_Pressed)
		{
			if (EventArgs.Key == EKeys::LeftMouseButton)
			{
				MousePressPosition = MousePos;
				bMouseDown = true;
			}

			HHitProxy*	HitResult = EventArgs.Viewport->GetHitProxy(HitX,HitY);
			
			// Short-term, performing a quick-out
			bool bHandledHitProxy = true;

			if (HitResult)
			{
				if (HitResult->IsA(HCascadeEdEmitterProxy::StaticGetType()))
				{
					UParticleEmitter* Emitter = ((HCascadeEdEmitterProxy*)HitResult)->Emitter;
					CascadePtr.Pin()->SetSelectedEmitter(Emitter);

					if (EventArgs.Key == EKeys::RightMouseButton)
					{
						OpenEmitterMenu();
					}
				}
				else if (HitResult->IsA(HCascadeEdEmitterEnableProxy::StaticGetType()))
				{
					if (bLODIsValid && (ParticleSystem != nullptr))
					{
						CascadePtr.Pin()->ToggleEnableOnSelectedEmitter(((HCascadeEdDrawModeButtonProxy*)HitResult)->Emitter);
					}
				}
				else if (HitResult->IsA(HCascadeEdDrawModeButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleEmitter*	Emitter		= ((HCascadeEdDrawModeButtonProxy*)HitResult)->Emitter;
						EEmitterRenderMode	DrawMode	= (EEmitterRenderMode)((HCascadeEdDrawModeButtonProxy*)HitResult)->DrawMode;

						switch (DrawMode)
						{
						case ERM_Normal:	DrawMode	= ERM_Point;	break;
						case ERM_Point:		DrawMode	= ERM_Cross;	break;
						case ERM_Cross:		DrawMode	= ERM_LightsOnly;		break;
						case ERM_LightsOnly:		DrawMode = ERM_None;		break;
						case ERM_None:		DrawMode = ERM_Normal;	break;
						}
						CascadePtr.Pin()->SetSelectedEmitter(Emitter);
						UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
						if (LODLevel && LODLevel->IsModuleEditable(LODLevel->RequiredModule))
						{
							Emitter->EmitterRenderMode	= DrawMode;
						}
					}
				}
				else if (HitResult->IsA(HCascadeEdSoloButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleEmitter* Emitter = ((HCascadeEdSoloButtonProxy*)HitResult)->Emitter;
						bool bIsSoloing = CascadePtr.Pin()->GetParticleSystem()->ToggleSoloing(Emitter);
						CascadePtr.Pin()->SetIsSoloing(bIsSoloing);
						CascadePtr.Pin()->SetSelectedEmitter(Emitter);
					}
				}
				else if (HitResult->IsA(HCascadeEdColorButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleEmitter* Emitter = ((HCascadeEdModuleProxy*)HitResult)->Emitter;
						UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;

						if (Module || Emitter)
						{
							TWeakObjectPtr<UObject> WeakTarget;
							TOptional<FLinearColor> InitialColor;
							if (Module)
							{
								InitialColor = Module->ModuleEditorColor;
								WeakTarget = Module;
							}
							else
							{
								check(Emitter);
								UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
								if ( LODLevel )
								{
									InitialColor = Emitter->EmitterEditorColor;
									WeakTarget = Emitter;
								}
							}

							if (InitialColor.IsSet())
							{
								// Let go of the mouse lock...
								EventArgs.Viewport->LockMouseToViewport(false);
								EventArgs.Viewport->CaptureMouse(false);

								FColorPickerArgs PickerArgs = FColorPickerArgs(InitialColor.GetValue(), FOnLinearColorValueChanged::CreateLambda([WeakTarget](FLinearColor NewValue)
									{
										if (UObject* Target = WeakTarget.Get())
										{
											if (UParticleModule* Module = Cast<UParticleModule>(Target))
											{
												Module->ModuleEditorColor = NewValue.ToFColorSRGB();
											}
											else if (UParticleEmitter* Emitter = CastChecked<UParticleEmitter>(Target))
											{
												Emitter->EmitterEditorColor = NewValue.ToFColorSRGB();
											}

										}
									}));
								PickerArgs.bClampValue = true;
								PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );

								OpenColorPicker(PickerArgs);
							}
						}
					}
				}
				else if (HitResult->IsA(HCascadeEdModuleProxy::StaticGetType()))
				{
					UParticleEmitter* Emitter = ((HCascadeEdModuleProxy*)HitResult)->Emitter;
					UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;

					CascadePtr.Pin()->SetSelectedModule(Emitter, Module);

					if (EventArgs.Key == EKeys::RightMouseButton)
					{
						if (bMouseDragging)// && (CurrentMoveMode != CMMM_None))
						{
							// Don't allow menu pop-up while moving modules...
						}
						else
						{
							OpenModuleMenu();
						}
					}
					else
					{
						check(CascadePtr.Pin()->GetSelectedModule());

						// We are starting to drag this module. Look at keys to see if we are moving/instancing
						if (bCtrlDown || bAltDown)
						{
							CascadePtr.Pin()->SetCopyModule(Emitter, Module);
							CurrentMoveMode = MoveMode_Copy;
						}
						else if (bShiftDown)
						{
							CurrentMoveMode = MoveMode_Instance;
						}
						else
						{
							CurrentMoveMode = MoveMode_Move;
						}

						// Figure out and save the offset from mouse location to top-left of selected module.
						FVector2D ModuleTopLeft(FindModuleTopLeft(Emitter, Module, EventArgs.Viewport));
						MouseHoldOffset = (ModuleTopLeft*GetDPIScale()- FVector2D(MousePressPosition)).IntPoint();
					}
				}
				else if (HitResult->IsA(HCascadeEdGraphButton::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleEmitter* Emitter = ((HCascadeEdModuleProxy*)HitResult)->Emitter;
						UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;

						if (Module)
						{
							CascadePtr.Pin()->SetSelectedModule(Emitter, Module);
						}
						else
						{
							CascadePtr.Pin()->SetSelectedEmitter(Emitter);
						}

						TArray<const FCurveEdEntry*> CurveEntries;
						const bool bNewCurve = CascadePtr.Pin()->AddSelectedToGraph(CurveEntries);
						if ( !bNewCurve )
						{
							CascadePtr.Pin()->ShowDesiredCurvesOnly(CurveEntries);
						}
					}
				}
				else if (HitResult->IsA(HCascadeEd3DDrawModeButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;
						check(Module);
						Module->b3DDrawMode = !Module->b3DDrawMode;
					}
				}
				else if (HitResult->IsA(HCascadeEd3DDrawModeOptionsButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;
						check(Module);
						// Pop up an options dialog??
						FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Prompt_7", "3DDrawMode Options Menu!"));
					}
				}
				else if (HitResult->IsA(HCascadeEdEnableButtonProxy::StaticGetType()))
				{
					if (bLODIsValid)
					{
						UParticleEmitter* Emitter = ((HCascadeEdModuleProxy*)HitResult)->Emitter;
						check(Emitter);
						UParticleModule* Module = ((HCascadeEdModuleProxy*)HitResult)->Module;
						check(Module);
						UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
						if (LODLevel && LODLevel->IsModuleEditable(Module))
						{
							Module->bEnabled = !Module->bEnabled;
							Module->PostEditChange();
							CascadePtr.Pin()->OnRestartInLevel();
						}
					}
				}
				else
				{
					bHandledHitProxy = false;
				}
			}
			else
			{
				bHandledHitProxy = false;
			}

			if (bHandledHitProxy == false)
			{
				CascadePtr.Pin()->SetSelectedModule(nullptr, nullptr);

				if (EventArgs.Key == EKeys::RightMouseButton)
				{
					OpenBackgroundMenu();
			}
			}

		}
		else if (EventArgs.Event == IE_Released)
		{
			// If we were dragging a module, find where the mouse currently is, and move module there
			if ((CurrentMoveMode != MoveMode_None) && bMouseDragging)
			{
				TArray<UParticleModule*>& ModuleDumpList = CascadePtr.Pin()->GetDraggedModuleList();
				if (DraggedModule)
				{
					// Find where to move module to.
					UParticleEmitter* TargetEmitter = NULL;
					int32 TargetIndex = INDEX_NONE;
					FindDesiredModulePosition(MousePos, TargetEmitter, TargetIndex);

					if (TargetEmitter && (TargetEmitter->bCollapsed == true))
					{
						TargetEmitter = NULL;
					}

					if (!TargetEmitter || TargetIndex == INDEX_NONE)
					{
						// If the target is the DumpModules area, add it to the list of dump modules
						if (bDrawDraggedModule)
						{
							ModuleDumpList.Add(DraggedModule);
							DraggedModule = nullptr;
						}
						else if (CurrentMoveMode == MoveMode_Move)
						{
							// If target is invalid and we were moving it, put it back where it came from.
							if ((ResetDragModIndex != INDEX_NONE) && CascadePtr.Pin()->GetSelectedEmitter())
							{
								CascadePtr.Pin()->InsertModule(DraggedModule, CascadePtr.Pin()->GetSelectedEmitter(), ResetDragModIndex, true);
								CascadePtr.Pin()->GetSelectedEmitter()->UpdateModuleLists();
								RemoveFromDraggedList(DraggedModule);
							}
							else
							{
								ModuleDumpList.Add(DraggedModule);
							}
						}
					}
					else
					{
						// Add dragged module in new location.
						if ((CurrentMoveMode == MoveMode_Move) || (CurrentMoveMode == MoveMode_Instance) || (CurrentMoveMode == MoveMode_Copy))
						{
							if (CurrentMoveMode == MoveMode_Copy)
							{
								CascadePtr.Pin()->CopyModuleToEmitter(DraggedModule, TargetEmitter, ParticleSystem, TargetIndex);
								TargetEmitter->UpdateModuleLists();
								RemoveFromDraggedList(DraggedModule);
							}
							else
							{
								if (CascadePtr.Pin()->InsertModule(DraggedModule, TargetEmitter, TargetIndex, true))
								{
									TargetEmitter->UpdateModuleLists();
								}
								else
								{
									CascadePtr.Pin()->InsertModule(DraggedModule, CascadePtr.Pin()->GetSelectedEmitter(), ResetDragModIndex, true);
									CascadePtr.Pin()->GetSelectedEmitter()->UpdateModuleLists();
								}
								RemoveFromDraggedList(DraggedModule);
							}

							CascadePtr.Pin()->OnRestartInLevel();
						}
					}
				}
			}

			bMouseDown = false;
			bMouseDragging = false;
			CurrentMoveMode = MoveMode_None;
			DraggedModule = nullptr;

			EventArgs.Viewport->Invalidate();
		}
		else if (EventArgs.Event == IE_DoubleClick)
		{
			if (EventArgs.Key == EKeys::LeftMouseButton)
			{
				HHitProxy*	HitResult = EventArgs.Viewport->GetHitProxy(HitX,HitY);
				if (HitResult)
				{
					if (HitResult->IsA(HCascadeEdEmitterProxy::StaticGetType()))
					{
						UParticleEmitter* Emitter = ((HCascadeEdEmitterProxy*)HitResult)->Emitter;
						if (Emitter)
						{
							Emitter->bCollapsed = !Emitter->bCollapsed;
							if (Emitter->bCollapsed == true)
							{
								CascadePtr.Pin()->SetSelectedModule(NULL);
							}
							EventArgs.Viewport->Invalidate();
						}
					}
				}
			}
		}
	}

	if (EventArgs.Event == IE_Pressed)
	{
		if (bMouseDragging && (CurrentMoveMode != MoveMode_None))
		{
			// Don't allow deleting while moving modules...
			bHandled = true;
		}
		else
		{
			if ( EventArgs.Key == EKeys::Platform_Delete )
			{
				if (CascadePtr.Pin()->GetSelectedModule())
				{
					CascadePtr.Pin()->OnDeleteModule(true);
				}
				else
				{
					CascadePtr.Pin()->OnDeleteEmitter();
				}
				bHandled = true;
			}
			else if (EventArgs.Key == EKeys::Left)
			{
				CascadePtr.Pin()->MoveSelectedEmitter(-1);
				bHandled = true;
			}
			else if (EventArgs.Key == EKeys::Right)
			{
				CascadePtr.Pin()->MoveSelectedEmitter(1);
				bHandled = true;
			}
			else if ((EventArgs.Key == EKeys::Z) && bCtrlDown)
			{
				CascadePtr.Pin()->OnUndo();
				bHandled = true;
			}
			else if ((EventArgs.Key == EKeys::Y) && bCtrlDown)
			{
				CascadePtr.Pin()->OnRedo();
				bHandled = true;
			}
			else if (EventArgs.Key == EKeys::PageDown)
			{
				CascadePtr.Pin()->OnJumpToLowerLOD();
				bHandled = true;
			}
			else if (EventArgs.Key == EKeys::PageUp)
			{
				CascadePtr.Pin()->OnJumpToHigherLOD();
				bHandled = true;
			}
		}
	}


	// Handle viewport screenshot.
	bHandled |= InputTakeScreenshot(EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	return bHandled;
}

void FCascadeEmitterCanvasClient::CapturedMouseMove(FViewport* InViewport, int32 X, int32 Y)
{
	// Update bMouseDragging.
	if (bMouseDown && !bMouseDragging)
	{
		UParticleEmitter* SelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();
		UParticleModule* SelectedModule = CascadePtr.Pin()->GetSelectedModule();
		int32 SelectedModuleIndex = CascadePtr.Pin()->GetSelectedModuleIndex();

		// See how far mouse has moved since we pressed button.
		FIntPoint TotalMouseMove = FIntPoint(X,Y) - MousePressPosition;

		int32 MoveThresh = CascadePtr.Pin()->GetEditorOptions() ? CascadePtr.Pin()->GetEditorOptions()->Cascade_MouseMoveThreshold : 4;
		MoveThresh = FMath::Max<int32>(4,MoveThresh);
		if (TotalMouseMove.SizeSquared() > FMath::Square(MoveThresh))
		{
			if ((SelectedModuleIndex == INDEX_REQUIREDMODULE) ||
				(SelectedModuleIndex == INDEX_SPAWNMODULE))
			{
				// Only allow dragging of these if they are being copied/shared...
				if ((InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl)) ||
					(InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift)))
				{
					bMouseDragging = true;
				}
			}
			else
			{
				bMouseDragging = true;
			}
		}

		if (SelectedEmitter)
		{
			int32 CurrentLODIndex = CascadePtr.Pin()->GetCurrentlySelectedLODLevelIndex();
			if (CurrentLODIndex != 0)
			{
				MousePressPosition = FIntPoint(X,Y);
				bMouseDragging = false;
			}
		}

		// If we are moving a module, here is where we remove it from its emitter.
		// Should not be able to change the CurrentMoveMode unless a module is selected.
		if (bMouseDragging && (CurrentMoveMode != MoveMode_None))
		{
			if (SelectedModule)
			{
				DraggedModule = SelectedModule;

				if (SelectedEmitter)
				{
					// DraggedModules
					if (DraggedModules.Num() == 0)
					{
						// We are pulling from an emitter...
						DraggedModules.InsertUninitialized(0, SelectedEmitter->LODLevels.Num());
					}

					for (int32 LODIndex = 0; LODIndex < SelectedEmitter->LODLevels.Num(); LODIndex++)
					{
						UParticleLODLevel* LODLevel = SelectedEmitter->LODLevels[LODIndex];
						if (LODLevel)
						{
							if (SelectedModuleIndex >= 0)
							{
								DraggedModules[LODIndex] = LODLevel->Modules[SelectedModuleIndex];
							}
							else
							{
								if (SelectedModuleIndex == INDEX_TYPEDATAMODULE)
								{
									DraggedModules[LODIndex]	= LODLevel->TypeDataModule;
								}
								else if (SelectedModuleIndex == INDEX_REQUIREDMODULE)
								{
									DraggedModules[LODIndex]	= LODLevel->RequiredModule;
								}
								else if (SelectedModuleIndex == INDEX_SPAWNMODULE)
								{
									DraggedModules[LODIndex]	= LODLevel->SpawnModule;
								}
							}
						}
					}
				}

				if (CurrentMoveMode == MoveMode_Move)
				{
					// Remember where to put this module back to if we abort the move.
					ResetDragModIndex = INDEX_NONE;
					if (SelectedEmitter)
					{
						UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel();
						if (LODLevel)
						{
							for (int32 i=0; i < LODLevel->Modules.Num(); i++)
							{
								if (LODLevel->Modules[i] == SelectedModule)
								{
									ResetDragModIndex = i;
								}
							}
						}

						if (ResetDragModIndex == INDEX_NONE)
						{
							if (SelectedModule->IsA(UParticleModuleTypeDataBase::StaticClass()))
							{
								ResetDragModIndex = INDEX_TYPEDATAMODULE;
							}
							else if (SelectedModule->IsA(UParticleModuleRequired::StaticClass()))
							{
								ResetDragModIndex = INDEX_REQUIREDMODULE;
							}
							else if (SelectedModule->IsA(UParticleModuleSpawn::StaticClass()))
							{
								ResetDragModIndex = INDEX_SPAWNMODULE;
							}
						}

						check(ResetDragModIndex != INDEX_NONE);
						if ((ResetDragModIndex != INDEX_SPAWNMODULE) &&
							(ResetDragModIndex != INDEX_REQUIREDMODULE))
						{
							CascadePtr.Pin()->OnDeleteModule(false);
						}
					}
					else
					{
						// Remove the module from the dump
						RemoveFromDraggedList(SelectedModule);
					}
				}
			}
		}
	}

	// If dragging a module around, update each frame.
	if (bMouseDragging && CurrentMoveMode != MoveMode_None)
	{
		InViewport->Invalidate();
	}
}

float FCascadeEmitterCanvasClient::GetViewportVerticalScrollBarRatio() const
{
	if (CanvasDimensions.Y == 0.0f)
	{
		return 1.0f;
	}

	float WidgetHeight = 1.0f;
	if (CascadeViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		WidgetHeight = CascadeViewportPtr.Pin()->GetViewport()->GetSizeXY().Y;
	}

	return WidgetHeight / CanvasDimensions.Y;
}

float FCascadeEmitterCanvasClient::GetViewportHorizontalScrollBarRatio() const
{
	float WidgetWidth = 1.0f;
	if (CascadeViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		WidgetWidth = CascadeViewportPtr.Pin()->GetViewport()->GetSizeXY().X;
	}

	return WidgetWidth / CanvasDimensions.X;
}

UParticleModule* FCascadeEmitterCanvasClient::GetDraggedModule()
{
	return DraggedModule;
}

TArray<UParticleModule*>& FCascadeEmitterCanvasClient::GetDraggedModules()
{
	return DraggedModules;
}

void FCascadeEmitterCanvasClient::UpdateScrollBars()
{
	CanvasDimensions.Y = 0;
	CanvasDimensions.X = 0;
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	for (int32 i = 0; i < ParticleSystem->Emitters.Num(); ++i)
	{
		int32 Height = 0;
		UParticleEmitter* Emitter = ParticleSystem->Emitters[i];

		if (Emitter)
		{
			if (Emitter->bCollapsed)
			{
				CanvasDimensions.X += EmitterCollapsedWidth;
			}
			else
			{
				CanvasDimensions.X += EmitterWidth;
			}

			Height = EmitterHeadHeight + ModulesOffset * ModuleHeight;

			UParticleEmitter* SaveSelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();
			CascadePtr.Pin()->SetSelectedEmitter(Emitter, true);
			UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel();
			if (LODLevel)
			{
				Height += LODLevel->Modules.Num() * ModuleHeight;
			}
			CascadePtr.Pin()->SetSelectedEmitter(SaveSelectedEmitter, true);

			if (Height > CanvasDimensions.Y)
			{
				CanvasDimensions.Y = Height;
			}
		}
	}
	CanvasDimensions.X += EmitterWidth; // Extra padding so the user can open a context menu in the "background" area

	if (CascadeViewportPtr.Pin()->GetVerticalScrollBar().IsValid() && CascadeViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		float VRatio = GetViewportVerticalScrollBarRatio();
		float HRatio = GetViewportHorizontalScrollBarRatio();
		float VDistFromBottom = CascadeViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float HDistFromBottom = CascadeViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromBottom();

		if (VRatio < 1.0f)
		{
			if (VDistFromBottom < 1.0f)
			{
				CascadeViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f), VRatio);
			}
			else
			{
				CascadeViewportPtr.Pin()->GetVerticalScrollBar()->SetState(0.0f, VRatio);
			}
		}

		if (HRatio < 1.0f)
		{
			if (HDistFromBottom < 1.0f)
			{
				CascadeViewportPtr.Pin()->GetHorizontalScrollBar()->SetState(FMath::Clamp(1.0f - HRatio - HDistFromBottom, 0.0f, 1.0f), HRatio);
			}
			else
			{
				CascadeViewportPtr.Pin()->GetHorizontalScrollBar()->SetState(0.0f, HRatio);
			}
		}
	}
}

void FCascadeEmitterCanvasClient::ChangeViewportScrollBarPosition(EScrollDirection Direction)
{
	if (CascadeViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		float Ratio = GetViewportVerticalScrollBarRatio();
		float DistFromBottom = CascadeViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float OneMinusRatio = 1.0f - Ratio;
		float Diff = 0.1f * OneMinusRatio;

		if (Direction == Scroll_Down)
		{
			Diff *= -1.0f;
		}

		CascadeViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(OneMinusRatio - DistFromBottom + Diff, 0.0f, OneMinusRatio), Ratio);

		CascadeViewportPtr.Pin()->RefreshViewport();
	}
}

FVector2D FCascadeEmitterCanvasClient::GetViewportScrollBarPositions() const
{
	FVector2D Positions = FVector2D::ZeroVector;
	if (CascadeViewportPtr.Pin()->GetVerticalScrollBar().IsValid() && CascadeViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		uint32 Width = CanvasDimensions.X;
		uint32 Height = CanvasDimensions.Y;
		float VRatio = GetViewportVerticalScrollBarRatio();
		float HRatio = GetViewportHorizontalScrollBarRatio();
		float VDistFromBottom = CascadeViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float HDistFromBottom = CascadeViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromBottom();

		if ((CascadeViewportPtr.Pin()->GetVerticalScrollBar()->GetVisibility() == EVisibility::Visible) && VDistFromBottom < 1.0f)
		{
			Positions.Y = FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f) * Height;
		}
		else
		{
			Positions.Y = 0.0f;
		}

		if ((CascadeViewportPtr.Pin()->GetHorizontalScrollBar()->GetVisibility() == EVisibility::Visible) && HDistFromBottom < 1.0f)
		{
			Positions.X = FMath::Clamp(1.0f - HRatio - HDistFromBottom, 0.0f, 1.0f) * Width;
		}
		else
		{
			Positions.X = 0.0f;
		}
	}

	return Positions;
}

void FCascadeEmitterCanvasClient::DrawEmitter(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport, FCanvas* Canvas)
{
	int32 ViewY = InViewport->GetSizeXY().Y;

	if (Emitter && (Emitter->bCollapsed == false))
	{
		// Draw background block

		// Draw header block
		DrawHeaderBlock(Index, XPos, Emitter, InViewport, Canvas);

		// Draw the type data module
		DrawTypeDataBlock(XPos, Emitter, InViewport, Canvas);

		// Draw the required module
		DrawRequiredBlock(XPos, Emitter, InViewport, Canvas);

		// Draw the spawn module
		DrawSpawnBlock(XPos, Emitter, InViewport, Canvas);

		// Draw each module - skipping the 'required' modules!
		int32 YPos = EmitterHeadHeight + ModulesOffset * ModuleHeight;
		int32 j;

		UParticleEmitter* SaveSelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();
		// Now, draw the remaining modules
		CascadePtr.Pin()->SetSelectedEmitter(Emitter, true);
		UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel();
		if (LODLevel)
		{
			for(j = 0; j < LODLevel->Modules.Num(); j++)
			{
				UParticleModule* Module = LODLevel->Modules[j];
				check(Module);
				if (!(Module->IsA(UParticleModuleTypeDataBase::StaticClass())))
				{
					DrawModule(XPos, YPos, Emitter, Module, InViewport, Canvas);
					// Update Y position for next module.
					YPos += ModuleHeight;
				}
			}
		}
		CascadePtr.Pin()->SetSelectedEmitter(SaveSelectedEmitter, true);
	}
	else
	{
		// Draw header block
		DrawCollapsedHeaderBlock(Index, XPos, Emitter, InViewport, Canvas);
	}
}

void FCascadeEmitterCanvasClient::DrawHeaderBlock(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport, FCanvas* Canvas)
{
	int32 ViewY = InViewport->GetSizeXY().Y;
	FColor HeadColor = (Emitter == CascadePtr.Pin()->GetSelectedEmitter()) ? EmitterSelectedColor : EmitterUnselectedColor;

	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
	if (LODLevel == NULL)
	{
		return;
	}

	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdEmitterProxy(Emitter));
	}

	// If the module is shared w/ higher LOD levels, then mark it as such...
	if (LODLevel->bEnabled == true)
	{
		Canvas->DrawTile(XPos, 0, EmitterWidth, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, HeadColor);
	}
	else
	{
		Canvas->DrawTile(XPos, 0, EmitterWidth, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, HeadColor, TexModuleDisabledBackground->GetResource());
	}

	UParticleSpriteEmitter* SpriteEmitter = Cast<UParticleSpriteEmitter>(Emitter);
	if (!Canvas->IsHitTesting())
	{
		if (SpriteEmitter)
		{
			FString TempString;

			TempString = SpriteEmitter->GetEmitterName().ToString();
			Canvas->DrawShadowedString(XPos + 10, 5, *TempString, GEngine->GetSmallFont(), FLinearColor::White);

			int32 ThumbSize = EmitterHeadHeight - 2*EmitterThumbBorder;
			FIntPoint ThumbPos(XPos + EmitterWidth - ThumbSize - EmitterThumbBorder, EmitterThumbBorder);
			ThumbPos.X += Origin2D.X;
			ThumbPos.Y += Origin2D.Y;

			UParticleLODLevel* HighestLODLevel = Emitter->LODLevels[0];

			TempString = FString::Printf(TEXT("%4d"), HighestLODLevel->PeakActiveParticles);
			Canvas->DrawShadowedString(XPos + 90, 25, *TempString, GEngine->GetSmallFont(), FLinearColor::White);

			if (!Canvas->IsHitTesting())
			{
				// Draw sprite material thumbnail.
				check(LODLevel->RequiredModule);

				UMaterialInterface* MaterialInterface = LODLevel->RequiredModule->Material;

				UParticleModuleTypeDataMesh* MeshTD = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule);
				if (MeshTD)
				{
					UStaticMesh* Mesh = MeshTD->Mesh;

					if (Mesh)
					{
						// See if there is a mesh material
						if (MeshTD->bOverrideMaterial == false)
						{
							MaterialInterface = Mesh->GetMaterial(0);
						}

						// See if there is a mesh material module...
						for (int32 ModIndex = 0; ModIndex < LODLevel->Modules.Num(); ModIndex++)
						{
							UParticleModuleMeshMaterial* MeshMatMod = Cast<UParticleModuleMeshMaterial>(LODLevel->Modules[ModIndex]);
							if (MeshMatMod && MeshMatMod->bEnabled)
							{
								for (int32 MatIndex = 0; MatIndex < MeshMatMod->MeshMaterials.Num(); MatIndex++)
								{
									if (MeshMatMod->MeshMaterials[MatIndex])
									{
										MaterialInterface = MeshMatMod->MeshMaterials[MatIndex];
										break;
									}
								}
							}
						}
					}
				}

				if (MaterialInterface)
				{	
					// Get the rendering info for this object
					FThumbnailRenderingInfo* RenderInfo =
						GUnrealEd->GetThumbnailManager()->GetRenderingInfo(MaterialInterface);
					// If there is an object configured to handle it, draw the thumbnail
					if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
					{
						FParticleEmitterThumbnail& Thumbnail = CascadePtr.Pin()->GetEmitterToThumbnailMap().FindOrAdd(Emitter);

						float DPIScale = Canvas->GetDPIScale();
						int32 ScaledSize = FMath::RoundToInt(ThumbSize*DPIScale);

						// Thumbnail size
						FIntPoint ThumbnailSize(ScaledSize, ScaledSize);

						const bool bNeedsResize = !Thumbnail.Texture || Thumbnail.Texture->GetSurfaceWidth() != ScaledSize || Thumbnail.Texture->GetSurfaceHeight() != ScaledSize;
						
						if (bNeedsResize)
						{
							if (!Thumbnail.Texture)
							{
								static const FName CascadeEmitterThumbnailName(TEXT("CascadeEmitterThumbnail"));
								Thumbnail.Texture = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), CascadeEmitterThumbnailName));
							}

							Thumbnail.Texture->InitCustomFormat(ScaledSize, ScaledSize, EPixelFormat::PF_B8G8R8A8, true);
							
						}

						// Always redraw for now to match legacy behavior.  Could cache for more efficiency. 
						const bool bNeedsRerender = true;// bNeedsResize || Thumbnail.Material != MaterialInterface;

						if (bNeedsRerender)
						{
							Thumbnail.Material = MaterialInterface;

							// Re-render
							FTextureRenderTargetResource* ThumbnailRTResource = Thumbnail.Texture->GameThread_GetRenderTargetResource();
							FCanvas ThumbnailCanvas(ThumbnailRTResource, nullptr, GetWorld(), GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, ShouldDPIScaleSceneCanvas() ? GetDPIScale() : 1.0f);
							RenderInfo->Renderer->Draw(MaterialInterface, 0, 0, ScaledSize, ScaledSize, ThumbnailRTResource, &ThumbnailCanvas, false);

							ENQUEUE_RENDER_COMMAND(TransitionThumbnail_RT)
							(
								[RenderTargetResource=ThumbnailRTResource](FRHICommandListImmediate& RHICmdList)
								{
									RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVMask));
								}
							);
						}
					
						Canvas->DrawTile(ThumbPos.X - Origin2D.X, ThumbPos.Y - Origin2D.Y, ThumbSize, ThumbSize, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, Thumbnail.Texture->GameThread_GetRenderTargetResource(), false);
					}
				}
				else
				{
					Canvas->DrawTile(ThumbPos.X - Origin2D.X, ThumbPos.Y - Origin2D.Y, ThumbSize, ThumbSize, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);		
				}
			}
		}
	}

	// Draw column background
	Canvas->DrawTile(XPos, EmitterHeadHeight, EmitterWidth, ViewY - EmitterHeadHeight - Origin2D.Y, 0.f, 0.f, 1.f, 1.f, EmitterBackgroundColor);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}

	// Draw enable/disable button
	FTexture* EnabledIconTxtr = NULL;
	if (LODLevel->bEnabled == true)
	{
		EnabledIconTxtr	= GetIconTexture(Icon_ModuleEnabled);
	}
	else
	{
		EnabledIconTxtr	= GetIconTexture(Icon_ModuleDisabled);
	}
	check(EnabledIconTxtr);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdEmitterEnableProxy(Emitter));
	}
	Canvas->DrawTile(XPos + 12, 26, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, EnabledIconTxtr);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}

	// Draw rendering mode button.
	FTexture* IconTxtr	= NULL;
	switch (SpriteEmitter->EmitterRenderMode)
	{
	case ERM_Normal:
		IconTxtr	= GetIconTexture(Icon_RenderNormal);
		break;
	case ERM_Point:
		IconTxtr	= GetIconTexture(Icon_RenderPoint);
		break;
	case ERM_Cross:
		IconTxtr	= GetIconTexture(Icon_RenderCross);
		break;
	case ERM_LightsOnly:
		IconTxtr = GetIconTexture(Icon_RenderLights);
		break;
	case ERM_None:
		IconTxtr	= GetIconTexture(Icon_RenderNone);
		break;
	}
	check(IconTxtr);

	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdDrawModeButtonProxy(Emitter, SpriteEmitter->EmitterRenderMode));
	}
	Canvas->DrawTile(XPos + 32, 26, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, IconTxtr);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}

	FTexture* SoloIconTxr = NULL;
	if (SpriteEmitter->bIsSoloing)
	{
		SoloIconTxr = GetIconTexture(Icon_SoloEnabled);
	}
	else
	{
		SoloIconTxr = GetIconTexture(Icon_SoloDisabled);
	}
	check(SoloIconTxr);

	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdSoloButtonProxy(Emitter));
	}
	Canvas->DrawTile(XPos + 52, 26, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, SoloIconTxr);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}

	DrawColorButton(XPos, Emitter, NULL, Canvas->IsHitTesting(), Canvas);
}

void FCascadeEmitterCanvasClient::DrawCollapsedHeaderBlock(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport,FCanvas* Canvas)
{
	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
	if (LODLevel == nullptr)
	{
		return;
	}

	int32 ViewY = InViewport->GetSizeXY().Y;
	FColor HeadColor = Emitter->EmitterEditorColor;

	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdEmitterProxy(Emitter));
	}

	// If the module is shared w/ higher LOD levels, then mark it as such...
	if (LODLevel->bEnabled == true)
	{
		Canvas->DrawTile(XPos, 0, EmitterCollapsedWidth, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, HeadColor);
	}
	else
	{
		Canvas->DrawTile(XPos, 0, EmitterCollapsedWidth, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, HeadColor, TexModuleDisabledBackground->GetResource());
	}

	// Draw column background
	Canvas->DrawTile(XPos, EmitterHeadHeight, EmitterCollapsedWidth, ViewY - EmitterHeadHeight - Origin2D.Y, 0.f, 0.f, 1.f, 1.f, EmitterBackgroundColor);
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
	}
}

void FCascadeEmitterCanvasClient::DrawTypeDataBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport, FCanvas* Canvas)
{
	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
	if (LODLevel)
	{
		UParticleModule* Module = LODLevel->TypeDataModule;
		if (Module)
		{
			check(Module->IsA(UParticleModuleTypeDataBase::StaticClass()));
			DrawModule(XPos, EmitterHeadHeight, Emitter, Module, InViewport, Canvas, false);
		}
	}
}

void FCascadeEmitterCanvasClient::DrawRequiredBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport, FCanvas* Canvas)
{
	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
	if (LODLevel)
	{
		check(LODLevel->RequiredModule);
		DrawModule(XPos, EmitterHeadHeight + RequiredModuleOffset * ModuleHeight, Emitter, LODLevel->RequiredModule, InViewport, Canvas, false);
	}
}

void FCascadeEmitterCanvasClient::DrawSpawnBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* InViewport, FCanvas* Canvas)
{
	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
	if (LODLevel)
	{
		UParticleModule* Module = LODLevel->SpawnModule;
		if (Module)
		{
			check(Module->IsA(UParticleModuleSpawn::StaticClass()));
			DrawModule(XPos, EmitterHeadHeight + SpawnModuleOffset * ModuleHeight, Emitter, Module, InViewport, Canvas);
		}
	}
}

void FCascadeEmitterCanvasClient::DrawModule(int32 XPos, int32 YPos, UParticleEmitter* Emitter, UParticleModule* Module, 
	FViewport* InViewport, FCanvas* Canvas, bool bDrawEnableButton)
{	
	// Hack to ensure no black modules...
	if (Module->ModuleEditorColor == FColor(0,0,0,0))
	{
		Module->ModuleEditorColor = FColor::MakeRandomColor();
	}

	// Grab the correct color to use
	FColor ModuleBkgColor;
	if (CascadePtr.Pin()->GetIsSoloing() && (Emitter->bIsSoloing == false))
	{
		ModuleBkgColor	= FColor(0,0,0,0);
	}
	else if (Module == CascadePtr.Pin()->GetSelectedModule())
	{
		ModuleBkgColor	= ModuleColors[Module->GetModuleType()][Selection_Selected];
	}
	else
	{
		ModuleBkgColor	= ModuleColors[Module->GetModuleType()][Selection_Unselected];
	}

	// Offset the 2D draw origin
	Canvas->PushRelativeTransform(FTranslationMatrix(FVector(XPos,YPos,0)));

	bool bCanvasHitTesting = Canvas->IsHitTesting();
	// Draw the module box and it's proxy
	DrawModule(Canvas, Module, ModuleBkgColor, Emitter);
	if (CascadePtr.Pin()->GetIsModuleShared(Module) || (CascadePtr.Pin()->GetCurveEditor().IsValid() && Module->IsDisplayedInCurveEd(CascadePtr.Pin()->GetCurveEditor()->GetEdSetup())))
	{
		DrawColorButton(XPos, Emitter, Module, bCanvasHitTesting, Canvas);
	}

	// Draw little 'send properties to graph' button.
	if (Module->ModuleHasCurves())
	{
		DrawCurveButton(Emitter, Module, bCanvasHitTesting, Canvas);
	}

	// Draw button for 3DDrawMode.
	if (CascadePtr.Pin()->GetEditorOptions()->bUseSlimCascadeDraw == false)
	{
		if (Module->bSupported3DDrawMode)
		{
			Draw3DDrawButton(Emitter, Module, bCanvasHitTesting, Canvas);
		}
	}

	if (bDrawEnableButton == true)
	{
		DrawEnableButton(Emitter, Module, bCanvasHitTesting, Canvas);
	}

	Canvas->PopTransform();
}

void FCascadeEmitterCanvasClient::DrawModule(FCanvas* Canvas, UParticleModule* Module, FColor ModuleBkgColor, UParticleEmitter* Emitter)
{
	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(new HCascadeEdModuleProxy(Emitter, Module));
	}

	Canvas->DrawTile(-1, -1, EmitterWidth+1, ModuleHeight+2, 0.f, 0.f, 0.f, 0.f, FLinearColor::Black);

	if (Canvas->IsHitTesting())
	{
		Canvas->SetHitProxy(NULL);
		return;
	}

	int32 CurrLODSetting	= CascadePtr.Pin()->GetCurrentlySelectedLODLevelIndex();
	UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);

	bool bIsModuleValid = true;
	{
		if (LODLevel)
		{
			// Excepting the spawn and required modules, check the type data filters.
			if (Module != LODLevel->SpawnModule
				&& Module != LODLevel->RequiredModule)
			{
				UCascadeConfiguration* EditorConfig = CascadePtr.Pin()->GetEditorConfiguration();
				FName TypeDataName = NAME_None;
				if (LODLevel->TypeDataModule)
				{
					TypeDataName = LODLevel->TypeDataModule->GetClass()->GetFName();
				}
				bIsModuleValid = EditorConfig->IsModuleTypeValid(TypeDataName, Module->GetClass()->GetFName());
			}
			if (bIsModuleValid)
			{
				FString ErrorString;
				bIsModuleValid = Module->IsValidForLODLevel(LODLevel,ErrorString);
				if (!ErrorString.IsEmpty())
				{
					new(ModuleErrorStrings) FString(FString::Printf(TEXT("%s: %s"), *Emitter->EmitterName.GetPlainNameString(), *ErrorString));
				}
			}
			else
			{
				NumRejectedModulesDrawn++;
			}
		}
	}

	// If the module is shared w/ higher LOD levels, then mark it as such...
	if (bIsModuleValid && LODLevel && LODLevel->IsModuleEditable(Module))
	{
		Canvas->DrawTile(0, 0, EmitterWidth-1, ModuleHeight, 0.f, 0.f, 1.f, 1.f, ModuleBkgColor);
	}
	else
	{
		FColor BkgColor = ModuleBkgColor;
		FTexture* BkgTexture = TexModuleDisabledBackground->GetResource();
		if (!bIsModuleValid)
		{
			BkgColor.R = 255;
			BkgTexture = GetIconTexture(Icon_ModuleDisabled);
		}
		Canvas->DrawTile(0, 0, EmitterWidth-1, ModuleHeight, 0.f, 0.f, 1.f, 1.f, BkgColor, BkgTexture);
	}

	int32 XL, YL;
	FString ModuleName = Module->GetClass()->GetDescription();

	// Postfix name with '+' if shared.
	if (CascadePtr.Pin()->GetIsModuleShared(Module))
		ModuleName = ModuleName + FString(TEXT("+"));

	StringSize(GEngine->GetSmallFont(), XL, YL, *(ModuleName));
	int32 StartY = 3;
	if (CascadePtr.Pin()->GetEditorOptions()->bCenterCascadeModuleText == true)
	{
		StartY = FMath::Max<int32>((ModuleHeight - YL) / 2, 3);
	}
	Canvas->DrawShadowedString(10, StartY, *(ModuleName), GEngine->GetSmallFont(), FLinearColor::White);
}

void FCascadeEmitterCanvasClient::DrawDraggedModule(UParticleModule* Module, FViewport* InViewport, FCanvas* Canvas)
{
	FIntPoint MousePos = FIntPoint(InViewport->GetMouseX(), InViewport->GetMouseY());

	// Draw indicator for where we would insert this module.
	UParticleEmitter* TargetEmitter = NULL;
	int32 TargetIndex = INDEX_NONE;
	FindDesiredModulePosition(MousePos, TargetEmitter, TargetIndex);

	// When dragging, draw the module under the mouse cursor.
	FVector Translate = FVector(MousePos.X +MouseHoldOffset.X, MousePos.Y+ MouseHoldOffset.Y, 0)/GetDPIScale();

	if (!Module->IsA(UParticleModuleTypeDataBase::StaticClass()))
	{
		if (Module->GetModuleType() == EPMT_Required)
		{
			Translate.Y += RequiredModuleOffset * ModuleHeight;
		}
		else if (Module->GetModuleType() == EPMT_Spawn)
		{
			Translate.Y += SpawnModuleOffset * ModuleHeight;
		}
		else
		{
			Translate.Y += ModulesOffset * ModuleHeight;
		}
	}

	Canvas->PushRelativeTransform(FTranslationMatrix(Translate));
	DrawModule(Canvas, DraggedModule, EmitterSelectedColor, TargetEmitter);
	Canvas->PopTransform();
}

void FCascadeEmitterCanvasClient::DrawCurveButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas)
{
	if (bHitTesting)
	{
		Canvas->SetHitProxy(new HCascadeEdGraphButton(Emitter, Module));
	}

	int32 YPosition = 2;
	if (CascadePtr.Pin()->GetEditorOptions()->bCenterCascadeModuleText == true)
	{
		YPosition = FMath::Max<int32>((ModuleHeight - 16) / 2, 2);
	}
	Canvas->DrawTile(EmitterWidth - 20, YPosition, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_CurveEdit));
	if (bHitTesting)
		Canvas->SetHitProxy(NULL);
}

void FCascadeEmitterCanvasClient::DrawColorButton(int32 XPos, UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas)
{
	if (bHitTesting)
		Canvas->SetHitProxy(new HCascadeEdColorButtonProxy(Emitter, Module));
	if (Module)
	{
		Canvas->DrawTile(0, 0, 5, ModuleHeight, 0.f, 0.f, 1.f, 1.f, Module->ModuleEditorColor);
	}
	else
	{
		Canvas->DrawTile(XPos, 0, 5, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, Emitter->EmitterEditorColor);
	}
	if (bHitTesting)
		Canvas->SetHitProxy(NULL);
}

void FCascadeEmitterCanvasClient::Draw3DDrawButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas)
{
	if (bHitTesting)
		Canvas->SetHitProxy(new HCascadeEd3DDrawModeButtonProxy(Emitter, Module));
	if (Module->b3DDrawMode)
	{
		Canvas->DrawTile(EmitterWidth - 40, 21, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_3DDrawEnabled));
	}
	else
	{
		Canvas->DrawTile(EmitterWidth - 40, 21, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_3DDrawDisabled));
	}
	if (bHitTesting)
		Canvas->SetHitProxy(NULL);

#if defined(_CASCADE_ALLOW_3DDRAWOPTIONS_)
	if (Module->b3DDrawMode)
	{
		if (bHitTesting)
			Canvas->SetHitProxy(new HCascadeEd3DDrawModeOptionsButtonProxy(Emitter, Module));
		Canvas->DrawTile(10 + 20, 20 + 10, 8, 8, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);
		Canvas->DrawTile(11 + 20, 21 + 10, 6, 6, 0.f, 0.f, 1.f, 1.f, FColor(100,200,100));
		if (bHitTesting)
			Canvas->SetHitProxy(NULL);
	}
#endif	//#if defined(_CASCADE_ALLOW_3DDRAWOPTIONS_)
}

void FCascadeEmitterCanvasClient::DrawEnableButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas)
{
	if (bHitTesting)
	{
		Canvas->SetHitProxy(new HCascadeEdEnableButtonProxy(Emitter, Module));
	}

	if (CascadePtr.Pin()->GetEditorOptions()->bUseSlimCascadeDraw == false)
	{
		if (Module->bEnabled)
		{
			Canvas->DrawTile(EmitterWidth - 20, 21, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_ModuleEnabled));
		}
		else
		{
			Canvas->DrawTile(EmitterWidth - 20, 21, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_ModuleDisabled));
		}
	}
	else
	{
		int32 YPosition = 2;
		if (CascadePtr.Pin()->GetEditorOptions()->bCenterCascadeModuleText == true)
		{
			YPosition = FMath::Max<int32>((ModuleHeight - 16) / 2, 2);
		}
		if (Module->bEnabled)
		{
			Canvas->DrawTile(EmitterWidth - 40, YPosition, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_ModuleEnabled));
		}
		else
		{
			Canvas->DrawTile(EmitterWidth - 40, YPosition, 16, 16, 0.f, 0.f, 1.f, 1.f, FLinearColor::White, GetIconTexture(Icon_ModuleDisabled));
		}
	}

	if (bHitTesting)
	{
		Canvas->SetHitProxy(NULL);
	}
}

void FCascadeEmitterCanvasClient::DrawModuleDump(FViewport* InViewport, FCanvas* Canvas)
{
#if defined(_CASCADE_ENABLE_MODULE_DUMP_)
	int32 ViewX = InViewport->GetSizeXY().X;
	int32 ViewY = InViewport->GetSizeXY().Y;
	bool bHitTesting = RI->IsHitTesting();
	int32 XPos = ViewX - EmitterWidth - 1;
	FColor HeadColor = EmitterUnselectedColor;

	FIntPoint SaveOrigin2D = RI->Origin2D;
	RI->SetOrigin2D(0, SaveOrigin2D.Y);

	Canvas->DrawTile(XPos - 2, 0, XPos + 2, ViewY - Origin2D.Y, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);
	Canvas->DrawTile(XPos, 0, EmitterWidth, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, HeadColor);
	Canvas->DrawTile(XPos, 0, 5, EmitterHeadHeight, 0.f, 0.f, 1.f, 1.f, FLinearColor::Black);

	FString ModuleDumpTitle = NSLOCTEXT("UnrealEd", "ModuleDump", "Module Dump").ToString();

	Canvas->DrawShadowedString(XPos + 10, 5, *ModuleDumpTitle, GEngine->GetSmallFont(), FLinearColor::White);

	// Draw column background
	Canvas->DrawTile(XPos, EmitterHeadHeight, EmitterWidth, ViewY - EmitterHeadHeight - Origin2D.Y, 0.f, 0.f, 1.f, 1.f, FColor(160, 160, 160));
	if (bHitTesting)
		Canvas->SetHitProxy(NULL);

	// Draw the dump module list...
	int32 YPos = EmitterHeadHeight;

	FVector2D TempOrigin = Origin2D;
	Origin2D.X = 0;
	for(int32 i = 0; i < CascadePtr.Pin()->ModuleDumpList.Num(); i++)
	{
		UParticleModule* Module = CascadePtr.Pin()->ModuleDumpList(i);
		check(Module);
		DrawModule(XPos, YPos, NULL, Module, InViewport, Canvas);
		// Update Y position for next module.
		YPos += ModuleHeight;
	}

	Origin2D.X = TempOrigin.X;
	RI->SetOrigin2D(SaveOrigin2D);
#endif	//#if defined(_CASCADE_ENABLE_MODULE_DUMP_)
}

void FCascadeEmitterCanvasClient::FindDesiredModulePosition(const FIntPoint& Pos, class UParticleEmitter* &OutEmitter, int32 &OutIndex)
{
	// Calculate the position on the canvas, not the window...
	float PositionCheck = Pos.X - Origin2D.X;
	float CurrentWidth = 0;
	int32 EmitterIndex = -1;
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();

	const float ScaledEmitterCollapsedWidth = EmitterCollapsedWidth * GetDPIScale();
	const float ScaledEmitterWidth = EmitterWidth * GetDPIScale();

	for (int32 CheckIndex = 0; CheckIndex < ParticleSystem->Emitters.Num(); CheckIndex++)
	{
		UParticleEmitter* CheckEmitter = ParticleSystem->Emitters[CheckIndex];
		if (CheckEmitter)
		{
			float CheckWidth = CheckEmitter->bCollapsed ? ScaledEmitterCollapsedWidth : ScaledEmitterWidth;
			if ((PositionCheck > CurrentWidth) && (PositionCheck <= CurrentWidth + CheckWidth))
			{
				EmitterIndex = CheckIndex;
				break;
			}
			CurrentWidth += CheckWidth;
		}
	}

	// If invalid Emitter, return nothing.
	if (EmitterIndex < 0 || EmitterIndex > ParticleSystem->Emitters.Num()-1)
	{
		OutEmitter = NULL;
		OutIndex = INDEX_NONE;
		return;
	}

	OutEmitter = ParticleSystem->Emitters[EmitterIndex];
	UParticleLODLevel* LODLevel	= OutEmitter->LODLevels[0];
	OutIndex = FMath::Clamp<int32>(((Pos.Y - Origin2D.Y) - (EmitterHeadHeight*GetDPIScale()) - ModulesOffset * (ModuleHeight*GetDPIScale())) / (ModuleHeight*GetDPIScale()), 0, LODLevel->Modules.Num());
}

FIntPoint FCascadeEmitterCanvasClient::FindModuleTopLeft(class UParticleEmitter* Emitter, class UParticleModule* Module, FViewport* InViewport)
{
	int32 i;
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();

	int32 EmitterIndex = -1;
	for(i = 0; i < ParticleSystem->Emitters.Num(); i++)
	{
		if (ParticleSystem->Emitters[i] == Emitter)
		{
			EmitterIndex = i;
		}
	}

	int32 ModuleIndex = 0;

	if (EmitterIndex != -1)
	{
		if (Module && Module->IsA(UParticleModuleTypeDataBase::StaticClass()))
		{
			return FIntPoint(EmitterIndex*EmitterWidth, EmitterHeadHeight);
		}
		else
		{
			UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel(Emitter);
			if (LODLevel)
			{
				for(i = 0; i < LODLevel->Modules.Num(); i++)
				{
					if (LODLevel->Modules[i] == Module)
					{
						ModuleIndex = i;
					}
				}
			}
		}

		int32 Width = 0;
		for (int32 InnerIndex = 0; InnerIndex < EmitterIndex; InnerIndex++)
		{
			UParticleEmitter* InnerEmitter = ParticleSystem->Emitters[InnerIndex];
			if (InnerEmitter)
			{
				Width += InnerEmitter->bCollapsed ? EmitterCollapsedWidth : EmitterWidth;
			}
		}
		return FIntPoint(Width, EmitterHeadHeight + ModuleIndex*ModuleHeight);
	}

	TArray<UParticleModule*>& ModuleDumpList = CascadePtr.Pin()->GetDraggedModuleList();
	// Must be in the module dump...
	check(ModuleDumpList.Num());
	for (i = 0; i < ModuleDumpList.Num(); i++)
	{
		if (ModuleDumpList[i] == Module)
		{
			int32 OffsetHeight = 0;
			if (!Module->IsA(UParticleModuleTypeDataBase::StaticClass()))
			{
				// When we grab from the dump, we need to account for no 'TypeData'
				OffsetHeight = ModuleHeight;
			}
			return FIntPoint(InViewport->GetSizeXY().X - EmitterWidth - Origin2D.X, EmitterHeadHeight - OffsetHeight + i * EmitterHeadHeight - Origin2D.Y);
		}
	}

	return FIntPoint(0.f, 0.f);
}

void FCascadeEmitterCanvasClient::RemoveFromDraggedList(UParticleModule* Module)
{
	TArray<UParticleModule*>& ModuleDumpList = CascadePtr.Pin()->GetDraggedModuleList();
	for (int32 i = 0; i < ModuleDumpList.Num(); i++)
	{
		if (ModuleDumpList[i] == Module)
		{
			ModuleDumpList.RemoveAt(i);
			break;
		}
	}
}

FTexture* FCascadeEmitterCanvasClient::GetIconTexture(ECascadeIcons eIcon)
{
	if ((eIcon >= 0) && (eIcon < Icon_COUNT))
	{
		UTexture2D* IconTexture = IconTex[eIcon];
		if (IconTexture)
		{
			return IconTexture->GetResource();
		}
	}

	check(!TEXT("Cascade: Invalid Icon Request!"));
	return NULL;
}

void FCascadeEmitterCanvasClient::OpenModuleMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		CascadeViewportPtr.Pin().ToSharedRef(),
		FWidgetPath(),
		BuildMenuWidgetModule(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void FCascadeEmitterCanvasClient::OpenEmitterMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		CascadeViewportPtr.Pin().ToSharedRef(),
		FWidgetPath(),
		BuildMenuWidgetEmitter(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

void FCascadeEmitterCanvasClient::OpenBackgroundMenu()
{
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

	FSlateApplication::Get().PushMenu(
		CascadeViewportPtr.Pin().ToSharedRef(),
		FWidgetPath(),
		BuildMenuWidgetBackround(),
		MouseCursorLocation,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
}

TSharedRef<SWidget> FCascadeEmitterCanvasClient::BuildMenuWidgetModule()
{
	UParticleModule* SelectedModule = CascadePtr.Pin()->GetSelectedModule();
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CascadePtr.Pin()->GetToolkitCommands());
	if (SelectedModule)
	{
		MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DeleteModule);
		MenuBuilder.AddMenuEntry(FCascadeCommands::Get().RefreshModule);
		
		if (SelectedModule != NULL)
		{
			if (SelectedModule->IsA(UParticleModuleRequired::StaticClass()))
			{
				MenuBuilder.BeginSection("CascadeSyncUseMaterial");
				{
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().SyncMaterial);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().UseMaterial);
				}
				MenuBuilder.EndSection();
			}

			int32 CurrLODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevelIndex();
			if (CurrLODLevel > 0)
			{
				bool bAddDuplicateOptions = true;
				if (CascadePtr.Pin()->GetIsModuleShared(SelectedModule) == true)
				{
					bAddDuplicateOptions = false;
				}

				if (bAddDuplicateOptions == true)
				{
					MenuBuilder.BeginSection("CascadeDupe");
					{
						MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DupeFromHigher);
						MenuBuilder.AddMenuEntry(FCascadeCommands::Get().ShareFromHigher);
						MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DupeFromHighest);
					}
					MenuBuilder.EndSection();
				}
				else
				{
					// It's shared... add an unshare option
				}
			}

			if (SelectedModule->SupportsRandomSeed())
			{
				MenuBuilder.BeginSection("CascadeRandomSeed");
				{
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().SetRandomSeed);
				}
				MenuBuilder.EndSection();
			}
			else
			{
				if (CurrLODLevel == 0)
				{
					// See if there is a seeded version of this module...
					FString ClassName = SelectedModule->GetClass()->GetName();
					UE_LOGF(LogCascade, Log, "Non-seeded module %ls", *ClassName);
					// This only works if the seeded version is names <ClassName>_Seeded!!!!
					FString SeededClassName = ClassName + TEXT("_Seeded");
					if (UClass::TryFindTypeSlow<UClass>(SeededClassName) != nullptr)
					{
						MenuBuilder.BeginSection("CascadeConvertToSeeded");
						{
							MenuBuilder.AddMenuEntry(FCascadeCommands::Get().ConvertToSeeded);
						}
						MenuBuilder.EndSection();
					}
				}
			}

			int32 CustomEntryCount = SelectedModule->GetNumberOfCustomMenuOptions();
			if (CustomEntryCount > 0)
			{
				MenuBuilder.BeginSection("CascadeCustomMenuOptions");
				{
					for (int32 EntryIdx = 0; EntryIdx < CustomEntryCount; EntryIdx++)
					{
						FString DisplayString;
						if (SelectedModule->GetCustomMenuEntryDisplayString(EntryIdx, DisplayString) == true)
						{
							MenuBuilder.AddMenuEntry(
								FText::FromString( DisplayString ),
								FText(),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnCustomModuleOption, EntryIdx)));
						}
					}
				}
				MenuBuilder.EndSection();
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FCascadeEmitterCanvasClient::BuildMenuWidgetEmitter()
{
	UParticleEmitter* SelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();
	UCascadeOptions* EditorOptions = CascadePtr.Pin()->GetEditorOptions();
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CascadePtr.Pin()->GetToolkitCommands());
	if (SelectedEmitter)
	{
		InitializeModuleEntries();

		// Emitter options
		{
			if (EditorOptions->bUseSubMenus == false)
			{
				MenuBuilder.BeginSection("CascadeEmitterOptionsNoSubMenus");
				{
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().RenameEmitter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DuplicateEmitter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DuplicateShareEmitter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().DeleteEmitter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().ExportEmitter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().ExportAllEmitters);
				}
				MenuBuilder.EndSection();
			}
			else
			{
				struct Local
				{
					static void BuildEmitterMenu(FMenuBuilder& Menu)
					{
						const FCascadeCommands& Actions = FCascadeCommands::Get();

						Menu.BeginSection("CascadeEmitter", NSLOCTEXT("Cascade", "EmitterHeader", "Emitter"));
						{
							Menu.AddMenuEntry(FCascadeCommands::Get().RenameEmitter);
							Menu.AddMenuEntry(FCascadeCommands::Get().DuplicateEmitter);
							Menu.AddMenuEntry(FCascadeCommands::Get().DuplicateShareEmitter);
							Menu.AddMenuEntry(FCascadeCommands::Get().DeleteEmitter);
							Menu.AddMenuEntry(FCascadeCommands::Get().ExportEmitter);
							Menu.AddMenuEntry(FCascadeCommands::Get().ExportAllEmitters);
						}
						Menu.EndSection();
					}
				};
				MenuBuilder.BeginSection("CascadeEmitterOptionsNoHeader");
				{
					MenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "EmitterSubMenu", "Emitter"), FText::GetEmpty(), FNewMenuDelegate::CreateStatic(&Local::BuildEmitterMenu));
				}
				MenuBuilder.EndSection();
			}
		}

		// Particle system
		{
			if (EditorOptions->bUseSubMenus == false)
			{
				MenuBuilder.BeginSection("CascadeParticleSystemNoSubMenus");
				{
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().SelectParticleSystem);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().NewEmitterBefore);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().NewEmitterAfter);
					MenuBuilder.AddMenuEntry(FCascadeCommands::Get().RemoveDuplicateModules);
				}
				MenuBuilder.EndSection();
			}
			else
			{
				struct Local
				{
					static void BuildParticleSystemMenu(FMenuBuilder& Menu)
					{
						const FCascadeCommands& Actions = FCascadeCommands::Get();

						Menu.BeginSection("CascadeParticleSystem", NSLOCTEXT("Cascade", "ParticleSystemHeader", "Particle System"));
						{
							Menu.AddMenuEntry(FCascadeCommands::Get().SelectParticleSystem);
							Menu.AddMenuEntry(FCascadeCommands::Get().NewEmitterBefore);
							Menu.AddMenuEntry(FCascadeCommands::Get().NewEmitterAfter);
							Menu.AddMenuEntry(FCascadeCommands::Get().RemoveDuplicateModules);
						}
						Menu.EndSection();
					}
				};
				MenuBuilder.BeginSection("CascadeParticleSystemNoHeader");
				{
					MenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "ParticleSystemSubMenu", "Particle System"), FText::GetEmpty(), FNewMenuDelegate::CreateStatic(&Local::BuildParticleSystemMenu));
				}
				MenuBuilder.EndSection();
			}
		}

		// New module data types
		{
			if (TypeDataModuleEntries.Num())
			{
				MenuBuilder.BeginSection("CascadeModuleDatTypes");
				{
					if (EditorOptions->bUseSubMenus == false)
					{
						// add the data type modules to the menu
						for (int32 i = 0; i < TypeDataModuleEntries.Num(); i++)
						{
							MenuBuilder.AddMenuEntry(
								FText::FromString( TypeDataModuleEntries[i] ),
								FText(),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnNewModule, TypeDataModuleIndices[i])));
						}
					}
					else
					{
						MenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "NewDataTypeSubMenu", "TypeData"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &FCascadeEmitterCanvasClient::BuildNewModuleDataTypeMenu));
					}
				}
				MenuBuilder.EndSection();
			}
		}

		// New Modules
		{
			if (ModuleEntries.Num())
			{
				if (EditorOptions->bUseSubMenus == false)
				{
					// Add each module type to menu.
					for (int32 i = 0; i < ModuleEntries.Num(); i++)
					{
						MenuBuilder.AddMenuEntry(
							FText::FromString( ModuleEntries[i] ),
							FText(),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnNewModule, ModuleIndices[i])));
					}
				}
				else
				{
					UCascadeConfiguration* EditorConfig = CascadePtr.Pin()->GetEditorConfiguration();
					TArray<UClass*> ParticleModuleBaseClasses = CascadePtr.Pin()->GetParticleModuleBaseClasses();
					TArray<UClass*> ParticleModuleClasses = CascadePtr.Pin()->GetParticleModuleClasses();
					bool bFoundTypeData = false;
					FString ModuleName;

					// Now, for each module base type, add another branch
					for (int32 i = 0; i < ParticleModuleBaseClasses.Num(); i++)
					{
						ModuleName = ParticleModuleBaseClasses[i]->GetName();
						if (IsModuleSuitableForModuleMenu(ModuleName) &&
							IsBaseModuleTypeDataPairSuitableForModuleMenu(ModuleName) &&
							HasValidChildModules(i))
						{
							MenuBuilder.AddSubMenu( FText::FromString( ParticleModuleBaseClasses[i]->GetDescription() ), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &FCascadeEmitterCanvasClient::BuildNewModuleSubMenu, i));
						}
					}
				}
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FCascadeEmitterCanvasClient::BuildMenuWidgetBackround()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CascadePtr.Pin()->GetToolkitCommands());
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), FText::FromString( UParticleSpriteEmitter::StaticClass()->GetDescription() ) );

		MenuBuilder.AddMenuEntry( FText::Format( NSLOCTEXT("Cascade", "NewSoundEmitter", "New {ClassName}"), Args ), 
			FText::GetEmpty(), 
			FSlateIcon(), 
			FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnNewEmitter)));
	}

	return MenuBuilder.MakeWidget();
}

void FCascadeEmitterCanvasClient::BuildNewModuleDataTypeMenu(FMenuBuilder& Menu)
{
	Menu.BeginSection("CascadeTypeData", NSLOCTEXT("Cascade", "NewDataTypeHeader", "TypeData"));
	{
		// add the data type modules to the menu
		for (int32 i = 0; i < TypeDataModuleEntries.Num(); i++)
		{
			Menu.AddMenuEntry(
				FText::FromString( TypeDataModuleEntries[i] ),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnNewModule, TypeDataModuleIndices[i])));
		}
	}
	Menu.EndSection();
}

bool FCascadeEmitterCanvasClient::HasValidChildModules(int32 i) const
{
	TArray<UClass*> ParticleModuleBaseClasses = CascadePtr.Pin()->GetParticleModuleBaseClasses();
	TArray<UClass*> ParticleModuleClasses = CascadePtr.Pin()->GetParticleModuleClasses();
	FString ModuleName;

	// Search for all modules of this type
	for (int32 j = 0; j < ParticleModuleClasses.Num(); j++)
	{
		if (ParticleModuleClasses[j]->IsChildOf(ParticleModuleBaseClasses[i]))
		{
			ModuleName = ParticleModuleClasses[j]->GetName();
			if (IsModuleSuitableForModuleMenu(ModuleName) &&
				IsModuleTypeDataPairSuitableForModuleMenu(ModuleName))
			{
				return true;
			}
		}
	}
	return false;
}

void FCascadeEmitterCanvasClient::BuildNewModuleSubMenu(FMenuBuilder& Menu, int32 i)
{
	TArray<UClass*> ParticleModuleBaseClasses = CascadePtr.Pin()->GetParticleModuleBaseClasses();
	TArray<UClass*> ParticleModuleClasses = CascadePtr.Pin()->GetParticleModuleClasses();
	FString ModuleName;

	// Search for all modules of this type
	for (int32 j = 0; j < ParticleModuleClasses.Num(); j++)
	{
		if (ParticleModuleClasses[j]->IsChildOf(ParticleModuleBaseClasses[i]))
		{
			ModuleName = ParticleModuleClasses[j]->GetName();
			if (IsModuleSuitableForModuleMenu(ModuleName) &&
				IsModuleTypeDataPairSuitableForModuleMenu(ModuleName))
			{
				Menu.AddMenuEntry(
					FText::FromString( ParticleModuleClasses[j]->GetDescription() ),
					FText(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(CascadePtr.Pin().ToSharedRef(), &FCascade::OnNewModule, j)));
			}
		}
	}
}

void FCascadeEmitterCanvasClient::InitializeModuleEntries()
{
	int32 i;
	bool bFoundTypeData = false;
	UParticleModule* DefModule;
	
	if (!bInitializedModuleEntries)
	{
		TArray<UClass*> ParticleModuleClasses = CascadePtr.Pin()->GetParticleModuleClasses();

		TypeDataModuleEntries.Empty();
		TypeDataModuleIndices.Empty();
		ModuleEntries.Empty();
		ModuleIndices.Empty();

		// add the data type modules to the menu
		for(i = 0; i < ParticleModuleClasses.Num(); i++)
		{
			DefModule = ParticleModuleClasses[i]->GetDefaultObject<UParticleModule>();
			FString ClassName = ParticleModuleClasses[i]->GetName();
			if (ParticleModuleClasses[i]->IsChildOf(UParticleModuleTypeDataBase::StaticClass()))
			{
				if (IsModuleSuitableForModuleMenu(ClassName))
				{
					bFoundTypeData = true;
					FString NewModuleString = FText::Format(NSLOCTEXT("UnrealEd", "New_F", "New {0}"), FText::FromString(ParticleModuleClasses[i]->GetDescription())).ToString();
					TypeDataModuleEntries.Add(NewModuleString);
					TypeDataModuleIndices.Add(i);
				}
			}
		}

		// Add each module type to menu.
		for(i = 0; i < ParticleModuleClasses.Num(); i++)
		{
			DefModule = ParticleModuleClasses[i]->GetDefaultObject<UParticleModule>();
			FString ClassName = ParticleModuleClasses[i]->GetName();
			if (ParticleModuleClasses[i]->IsChildOf(UParticleModuleTypeDataBase::StaticClass()) == false)
			{
				if (IsModuleSuitableForModuleMenu(ClassName))
				{
					FString NewModuleString = FText::Format(NSLOCTEXT("UnrealEd", "New_F", "New {0}"), FText::FromString(ParticleModuleClasses[i]->GetDescription())).ToString();
					ModuleEntries.Add(NewModuleString);
					ModuleIndices.Add(i);
				}
			}
		}
		bInitializedModuleEntries = true;
	}
}

bool FCascadeEmitterCanvasClient::IsModuleSuitableForModuleMenu(FString& InModuleName) const
{
	int32 RejectIndex;
	UCascadeConfiguration* EditorConfig = CascadePtr.Pin()->GetEditorConfiguration();
	return (EditorConfig->ModuleMenu_ModuleRejections.Find(InModuleName, RejectIndex) == false);
}

bool FCascadeEmitterCanvasClient::IsBaseModuleTypeDataPairSuitableForModuleMenu(FString& InModuleName) const
{
	int32 RejectIndex;
	UCascadeConfiguration* EditorConfig = CascadePtr.Pin()->GetEditorConfiguration();
	UParticleEmitter* SelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();

	FString TDName(TEXT("None"));
	if (SelectedEmitter)
	{
		UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel();
		if (LODLevel && LODLevel->TypeDataModule)
		{
			TDName = LODLevel->TypeDataModule->GetClass()->GetName();
		}
	}

	FModuleMenuMapper* Mapper = NULL;
	for (int32 MapIndex = 0; MapIndex < EditorConfig->ModuleMenu_TypeDataToBaseModuleRejections.Num(); MapIndex++)
	{
		if (EditorConfig->ModuleMenu_TypeDataToBaseModuleRejections[MapIndex].ObjName == TDName)
		{
			Mapper = &(EditorConfig->ModuleMenu_TypeDataToBaseModuleRejections[MapIndex]);
			break;
		}
	}

	if (Mapper)
	{
		if (Mapper->InvalidObjNames.Find(InModuleName, RejectIndex) == true)
		{
			return false;
		}
	}

	return true;
}

bool FCascadeEmitterCanvasClient::IsModuleTypeDataPairSuitableForModuleMenu(FString& InModuleName) const
{
	int32 RejectIndex;
	UCascadeConfiguration* EditorConfig = CascadePtr.Pin()->GetEditorConfiguration();
	UParticleEmitter* SelectedEmitter = CascadePtr.Pin()->GetSelectedEmitter();

	FString TDName(TEXT("None"));
	if (SelectedEmitter)
	{
		UParticleLODLevel* LODLevel = CascadePtr.Pin()->GetCurrentlySelectedLODLevel();
		if (LODLevel && LODLevel->TypeDataModule)
		{
			TDName = LODLevel->TypeDataModule->GetClass()->GetName();
		}
	}

	FModuleMenuMapper* Mapper = NULL;
	for (int32 MapIndex = 0; MapIndex < EditorConfig->ModuleMenu_TypeDataToSpecificModuleRejections.Num(); MapIndex++)
	{
		if (EditorConfig->ModuleMenu_TypeDataToSpecificModuleRejections[MapIndex].ObjName == TDName)
		{
			Mapper = &(EditorConfig->ModuleMenu_TypeDataToSpecificModuleRejections[MapIndex]);
			break;
		}
	}

	if (Mapper)
	{
		if (Mapper->InvalidObjNames.Find(InModuleName, RejectIndex) == true)
		{
			return false;
		}
	}

	return true;
}

float FCascadeEmitterCanvasClient::UpdateViewportClientWindowDPIScale()  const
{
	float DPIScale = 1.f;
	if (CascadeViewportPtr.IsValid())
	{
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(CascadeViewportPtr.Pin().ToSharedRef());
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}
	}

	return DPIScale;
}
