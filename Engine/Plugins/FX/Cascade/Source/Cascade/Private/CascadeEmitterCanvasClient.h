// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EditorViewportClient.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Particles/ParticleModule.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FCanvas;
class FCascade;
class FMenuBuilder;
class FTexture;
class FViewport;
class SCascadeEmitterCanvas;
class SWidget;
class UParticleEmitter;
class UTexture2D;
struct FInputKeyEventArgs;

/*-----------------------------------------------------------------------------
   FCascadeCanvasClient
-----------------------------------------------------------------------------*/

class FCascadeEmitterCanvasClient : public FEditorViewportClient
{
public:
	/** Constructor */
	FCascadeEmitterCanvasClient(TSharedPtr<FCascade> InCascade, TWeakPtr<SCascadeEmitterCanvas> InCascadeViewport);
	~FCascadeEmitterCanvasClient();

	/** FViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void CapturedMouseMove(FViewport* Viewport, int32 X, int32 Y) override;

	/** Returns the ratio of the size of the particle emitters to the size of the viewport */
	float GetViewportVerticalScrollBarRatio() const;
	float GetViewportHorizontalScrollBarRatio() const;

	/** Accessors */
	UParticleModule* GetDraggedModule();
	TArray<UParticleModule*>& GetDraggedModules();

private:
	/** Type of move operation */
	enum ECascadeModuleMoveMode
	{
		MoveMode_None,
		MoveMode_Move,
		MoveMode_Instance,
		MoveMode_Copy
	};

	/** Emitter/module icon types */
	enum ECascadeIcons
	{
		Icon_RenderNormal	= 0,
		Icon_RenderCross,
		Icon_RenderPoint,
		Icon_RenderNone,
		Icon_RenderLights,
		Icon_CurveEdit,
		Icon_3DDrawEnabled,
		Icon_3DDrawDisabled,
		Icon_ModuleEnabled,
		Icon_ModuleDisabled,
		Icon_SoloEnabled,
		Icon_SoloDisabled,
		Icon_COUNT
	};

	/** Module selection type */
	enum ECascadeModuleSelection
	{
		Selection_Unselected = 0,
		Selection_Selected = 1,
		Selection_COUNT = 2
	};

	/** Updates the states of the scrollbars */
	void UpdateScrollBars();

	/** Changes the position of the vertical scrollbar (on a mouse scrollwheel event) */
	void ChangeViewportScrollBarPosition(EScrollDirection Direction);

	/** Returns the positions of the scrollbars relative to the Particle Particles */
	FVector2D GetViewportScrollBarPositions() const;

	/** Custom draw methods for various emitter/module elements */
	void DrawEmitter(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawHeaderBlock(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawCollapsedHeaderBlock(int32 Index, int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawTypeDataBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawRequiredBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawSpawnBlock(int32 XPos, UParticleEmitter* Emitter, FViewport* Viewport, FCanvas* Canvas);
	void DrawModule(int32 XPos, int32 YPos, UParticleEmitter* Emitter, UParticleModule* Module, FViewport* Viewport, FCanvas* Canvas, bool bDrawEnableButton = true);
	void DrawModule(FCanvas* Canvas, UParticleModule* Module, FColor ModuleBkgColor, UParticleEmitter* Emitter);
	void DrawDraggedModule(UParticleModule* Module, FViewport* Viewport, FCanvas* Canvas);
	void DrawCurveButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas);
	void DrawColorButton(int32 XPos, UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas);
	void Draw3DDrawButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas);
	void DrawEnableButton(UParticleEmitter* Emitter, UParticleModule* Module, bool bHitTesting, FCanvas* Canvas);
	void DrawModuleDump(FViewport* Viewport, FCanvas* Canvas);

	/** Helper methods */
	void FindDesiredModulePosition(const FIntPoint& Pos, class UParticleEmitter* &OutEmitter, int32 &OutIndex);
	FIntPoint FindModuleTopLeft(class UParticleEmitter* Emitter, class UParticleModule* Module, FViewport* Viewport);

	/** Removes the specified module from the dragged module list */
	void RemoveFromDraggedList(UParticleModule* Module);

	/** Returns an icon texture */
	FTexture* GetIconTexture(ECascadeIcons eIcon);

	/** Methods for building the various context menus */
	void OpenModuleMenu();
	void OpenEmitterMenu();
	void OpenBackgroundMenu();
	TSharedRef<SWidget> BuildMenuWidgetModule();
	TSharedRef<SWidget> BuildMenuWidgetEmitter();
	TSharedRef<SWidget> BuildMenuWidgetBackround();
	void BuildNewModuleDataTypeMenu(FMenuBuilder& Menu);
	void BuildNewModuleSubMenu(FMenuBuilder& Menu, int32 i);

	/** Checks to see if the base module [i] has any valid children */
	bool HasValidChildModules(int32 i) const;

	/** Initializes the data used in building the module type context menu options */
	void InitializeModuleEntries();

	/** Is the module of the given name suitable for the right-click module menu? */
	bool IsModuleSuitableForModuleMenu(FString& InModuleName) const;

	/** Is the base module of the given name suitable for the right-click module menu given the currently selected emitter TypeData? */
	bool IsBaseModuleTypeDataPairSuitableForModuleMenu(FString& InModuleName) const;

	/** Is the base module of the given name suitable for the right-click module menu given the currently selected emitter TypeData? */
	bool IsModuleTypeDataPairSuitableForModuleMenu(FString& InModuleName) const;

	/** FViewport interface */
	virtual float UpdateViewportClientWindowDPIScale() const override;
private:
	/** Pointer back to the Particle editor tool that owns us */
	TWeakPtr<FCascade> CascadePtr;

	/** Pointer back to the Particle viewport control that owns us */
	TWeakPtr<SCascadeEmitterCanvas> CascadeViewportPtr;

	/** Size of the viewport canvas */
	FIntPoint CanvasDimensions;

	/** User input state info */
	ECascadeModuleMoveMode CurrentMoveMode;
	FIntPoint MouseHoldOffset;
	FIntPoint MousePressPosition; 
	bool bMouseDragging;
	bool bMouseDown;

	/** Canvas offset */
	FIntPoint Origin2D;
	
	/** Currently dragged module. */
	UParticleModule* DraggedModule;
	TArray<UParticleModule*> DraggedModules;
	bool bDrawDraggedModule;

	/** If we abort a drag - here is where put the module back to (in the selected Emitter) */
	int32 ResetDragModIndex;

	/** Textures/icons */
	UTexture2D* IconTex[Icon_COUNT];
	UTexture2D* TexModuleDisabledBackground;

	/** Miscellaneous draw info */
	const int32	EmitterWidth;
	const int32	EmitterCollapsedWidth;
	const int32	EmitterHeadHeight;
	const int32	EmitterThumbBorder;
	int32 ModuleHeight;
	FColor ModuleColors[EPMT_MAX][Selection_COUNT];
	FColor EmptyBackgroundColor;
	FColor EmitterBackgroundColor;
	FColor EmitterSelectedColor;
	FColor EmitterUnselectedColor;
	const FColor RenderModeSelected;
	const FColor RenderModeUnselected;
	const FColor Module3DDrawModeEnabledColor;
	const FColor Module3DDrawModeDisabledColor;
	const int32 RequiredModuleOffset;
	const int32 SpawnModuleOffset;
	const int32 ModulesOffset;
	int32 NumRejectedModulesDrawn;
	TArray<FString> ModuleErrorStrings;

	/** Data used in building the module type context menu options */
	bool bInitializedModuleEntries;
	TArray<FString>	TypeDataModuleEntries;
	TArray<int32> TypeDataModuleIndices;
	TArray<FString> ModuleEntries;
	TArray<int32> ModuleIndices;
};
