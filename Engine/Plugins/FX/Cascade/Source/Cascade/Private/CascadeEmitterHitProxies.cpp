// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadeEmitterHitProxies.h"

IMPLEMENT_HIT_PROXY(HCascadeEdEmitterProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdModuleProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdEmitterEnableProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdDrawModeButtonProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdSoloButtonProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdGraphButton,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdColorButtonProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEd3DDrawModeButtonProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEdEnableButtonProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HCascadeEd3DDrawModeOptionsButtonProxy,HHitProxy);
