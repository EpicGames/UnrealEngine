// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HitProxies.h"

/*-----------------------------------------------------------------------------
   Hit Proxies
-----------------------------------------------------------------------------*/

struct HCascadeEdEmitterProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;

	HCascadeEdEmitterProxy(class UParticleEmitter* InEmitter) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter)
	{}
};

struct HCascadeEdModuleProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEdModuleProxy(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};

struct HCascadeEdEmitterEnableProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;

	HCascadeEdEmitterEnableProxy(class UParticleEmitter* InEmitter) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter)
	{}
};

struct HCascadeEdDrawModeButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	int32 DrawMode;

	HCascadeEdDrawModeButtonProxy(class UParticleEmitter* InEmitter, int32 InDrawMode) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		DrawMode(InDrawMode)
	{}
};

struct HCascadeEdSoloButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;

	HCascadeEdSoloButtonProxy(class UParticleEmitter* InEmitter) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter)
	{}
};

struct HCascadeEdGraphButton : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEdGraphButton(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};

struct HCascadeEdColorButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEdColorButtonProxy(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};

struct HCascadeEd3DDrawModeButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEd3DDrawModeButtonProxy(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};

struct HCascadeEdEnableButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEdEnableButtonProxy(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};

struct HCascadeEd3DDrawModeOptionsButtonProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	class UParticleEmitter* Emitter;
	class UParticleModule* Module;

	HCascadeEd3DDrawModeOptionsButtonProxy(class UParticleEmitter* InEmitter, class UParticleModule* InModule) :
	HHitProxy(HPP_UI),
		Emitter(InEmitter),
		Module(InModule)
	{}
};
