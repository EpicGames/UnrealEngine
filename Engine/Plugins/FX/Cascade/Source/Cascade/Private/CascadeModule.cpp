// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadeModule.h"

#include "Cascade.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Modules/ModuleManager.h"
#include "ParticleHelper.h"
#include "Particles/ParticleSystem.h"
#include "Particles/ParticleSystemComponent.h"

class IToolkitHost;

const FName CascadeAppIdentifier = FName(TEXT("CascadeApp"));

/*-----------------------------------------------------------------------------
   FCascadeModule
-----------------------------------------------------------------------------*/

class FCascadeModule : public ICascadeModule
{
public:
	/** Constructor, set up console commands and variables **/
	FCascadeModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		UParticleSystemComponent::OnSystemPreActivationChange.AddStatic(&FCascade::OnComponentActivationChange);
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
	}

	virtual TSharedRef<ICascade> CreateCascade(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UParticleSystem* ParticleSystem) override
	{
		TSharedRef<FCascade> NewCascade(new FCascade());
		NewCascade->InitCascade(Mode, InitToolkitHost, ParticleSystem);
		CascadeToolkits.Add(&*NewCascade);
		return NewCascade;
	}

	virtual void CascadeClosed(FCascade* CascadeInstance) override
	{
		CascadeToolkits.Remove(CascadeInstance);
	}

	virtual void RefreshCascade(UParticleSystem* ParticleSystem) override
	{
		for (int32 Idx = 0; Idx < CascadeToolkits.Num(); ++Idx)
		{
			if (CascadeToolkits[Idx]->GetParticleSystem() == ParticleSystem)
			{
				CascadeToolkits[Idx]->ForceUpdate();
			}
		}
	}

	virtual void ConvertModulesToSeeded(UParticleSystem* ParticleSystem) override
	{
		//Have to reset all existing component using this system.
		FParticleResetContext ResetCtx;
		ResetCtx.AddTemplate(ParticleSystem);

		FCascade::ConvertAllModulesToSeeded( ParticleSystem );
	}

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	/** List of open Cascade toolkits */
	TArray<FCascade*> CascadeToolkits;
};

IMPLEMENT_MODULE(FCascadeModule, Cascade);
