// Copyright Epic Games, Inc. All Rights Reserved.

#include "CascadePreviewViewportClient.h"
#include "EngineGlobals.h"
#include "SceneView.h"
#include "UObject/Package.h"
#include "Engine/Engine.h"
#include "Components/StaticMeshComponent.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Serialization/ArchiveCountMem.h"
#include "Preferences/CascadeOptions.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "ParticleHelper.h"
#include "Engine/StaticMesh.h"
#include "ImageUtils.h"
#include "SEditorViewport.h"
#include "Components/VectorFieldComponent.h"
#include "CascadeParticleSystemComponent.h"
#include "Cascade.h"
#include "SCascadePreviewViewport.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataGpu.h"
#include "Particles/VectorField/ParticleModuleVectorFieldLocal.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Components/LineBatchComponent.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceScene.h"

#define LOCTEXT_NAMESPACE "CascadeViewportClient"

FCascadeEdPreviewViewportClient::FCascadeEdPreviewViewportClient(TWeakPtr<FCascade> InCascade, const TSharedRef<SCascadePreviewViewport>& InCascadeViewport)
	: FEditorViewportClient(nullptr, nullptr, StaticCastSharedRef<SEditorViewport>(InCascadeViewport))
	, CascadePtr(InCascade)
	, CascadePreviewScene(FPreviewScene::ConstructionValues()
						.SetLightRotation(FRotator(-45.f, 180.f, 0.f))
						.SetSkyBrightness(0.25f)
						.SetLightBrightness(1.f))
	, VectorFieldHitproxyInfo(0)
	, LightRotSpeed(0.22f)
{
	PreviewScene = &CascadePreviewScene;
	check(CascadePtr.IsValid() && EditorViewportWidget.IsValid());

	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	UCascadeParticleSystemComponent* ParticleSystemComponent = CascadePtr.Pin()->GetParticleSystemComponent();
	UVectorFieldComponent* LocalVectorFieldPreviewComponent = CascadePtr.Pin()->GetLocalVectorFieldComponent();
	UCascadeOptions* EditorOptions = CascadePtr.Pin()->GetEditorOptions();

	check(EditorOptions);

	// Create ParticleSystemComponent to use for preview.
	ParticleSystemComponent->CascadePreviewViewportPtr = this;
	ParticleSystemComponent->CastShadow = true;
	CascadePreviewScene.AddComponent(ParticleSystemComponent, FTransform::Identity);
	ParticleSystemComponent->SetFlags(RF_Transactional);

	// Create a component for previewing local vector fields.
	LocalVectorFieldPreviewComponent->bPreviewVectorField = true;
	LocalVectorFieldPreviewComponent->SetVisibility(false);
	CascadePreviewScene.AddComponent(LocalVectorFieldPreviewComponent,FTransform::Identity);

	// Use game defaults to hide emitter sprite etc., but we want to still show the Axis widget in the corner...
	// todo: seems this cold be done cleaner
	EngineShowFlags = FEngineShowFlags(ESFIM_Game);
	EngineShowFlags.Game = 0;
	EngineShowFlags.SetSnap(0);

	SetViewMode(VMI_Lit);

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.SeparateTranslucency = true;

	OverrideNearClipPlane(1.0f);

	SetViewLocation( FVector(-200.f, 0.f, 0.f) );
	SetViewRotation( FRotator::ZeroRotator );	

	PreviewAngle = FRotator::ZeroRotator;
	PreviewDistance = 0.f;
	bCaptureScreenShot = false;

	BackgroundColor = FColor::Black;

	WidgetAxis = EAxisList::None;
	WidgetMM = WMM_Translate;
	bManipulatingVectorField = false;

	DrawFlags = ParticleCounts | ParticleSystemCompleted;
	
	bUsingOrbitCamera = true;

	WireSphereRadius = 150.0f;

	FColor GridColorAxis(0, 0, 80);
	FColor GridColorMajor(0, 0, 72);
	FColor GridColorMinor(0, 0, 64);

	GridColorAxis = CascadePtr.Pin()->GetEditorOptions()->GridColor_Hi;
	GridColorMajor = CascadePtr.Pin()->GetEditorOptions()->GridColor_Low;
	GridColorMinor = CascadePtr.Pin()->GetEditorOptions()->GridColor_Low;

	DrawHelper.bDrawGrid = CascadePtr.Pin()->GetEditorOptions()->bShowGrid;
	DrawHelper.GridColorAxis = GridColorAxis;
	DrawHelper.GridColorMajor = GridColorMajor;
	DrawHelper.GridColorMinor = GridColorMinor;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawPivot = false;
	DrawHelper.PerspectiveGridSize = CascadePtr.Pin()->GetEditorOptions()->GridPerspectiveSize;
	DrawHelper.DepthPriorityGroup = SDPG_World;
	
	if (DrawHelper.bDrawGrid)
	{
		EngineShowFlags.SetGrid(true);
	}

	if (EditorOptions->FloorMesh == TEXT(""))
	{
		if (ParticleSystem != NULL)
		{
			EditorOptions->FloorMesh = ParticleSystem->FloorMesh;
			EditorOptions->FloorScale = ParticleSystem->FloorScale;
			EditorOptions->FloorScale3D = ParticleSystem->FloorScale3D;
		}
		else
		{
			EditorOptions->FloorMesh = FString::Printf(TEXT("/Engine/EditorMeshes/AnimTreeEd_PreviewFloor.AnimTreeEd_PreviewFloor"));
			EditorOptions->FloorScale = 1.0f;
			EditorOptions->FloorScale3D = FVector(1.0f, 1.0f, 1.0f);
		}
		EditorOptions->bShowFloor = false;
	}

	UStaticMesh* Mesh = NULL;
	FloorComponent = NULL;
	if (ParticleSystem)
	{
		Mesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,
			*(ParticleSystem->FloorMesh),NULL,LOAD_None,NULL);
	}
	if ((Mesh == NULL) && (EditorOptions->FloorMesh != TEXT("")))
	{
		Mesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,
			*(EditorOptions->FloorMesh),NULL,LOAD_None,NULL);
	}
	if (Mesh == NULL)
	{
		// Safety catch...
		EditorOptions->FloorMesh = FString::Printf(TEXT("/Engine/EditorMeshes/AnimTreeEd_PreviewFloor.AnimTreeEd_PreviewFloor"));
		Mesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,
			*(EditorOptions->FloorMesh),NULL,LOAD_None,NULL);
	}

	if (Mesh)
	{
		FloorComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), TEXT("FloorComponent"));
		check(FloorComponent);
		FloorComponent->SetStaticMesh(Mesh);
		FloorComponent->DepthPriorityGroup = SDPG_World;

		// Hide it for now...
		FloorComponent->SetVisibility(EditorOptions->bShowFloor);
		if (ParticleSystem)
		{
			FloorComponent->SetRelativeLocation_Direct(ParticleSystem->FloorPosition);
			FloorComponent->SetRelativeRotation_Direct(ParticleSystem->FloorRotation);
			FloorComponent->SetRelativeScale3D(ParticleSystem->FloorScale3D);
		}
		else
		{
			FloorComponent->SetRelativeLocation_Direct(EditorOptions->FloorPosition);
			FloorComponent->SetRelativeRotation_Direct(EditorOptions->FloorRotation);
			FloorComponent->SetRelativeScale3D(EditorOptions->FloorScale3D);
		}

		FPhysScene* PhysScene = new FPhysScene();
		CascadePreviewScene.GetWorld()->SetPhysicsScene(PhysScene);

		CascadePreviewScene.AddComponent(FloorComponent,FTransform::Identity);
	}
}

void FCascadeEdPreviewViewportClient::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEditorViewportClient::AddReferencedObjects(Collector);

	CascadePreviewScene.AddReferencedObjects(Collector);
}

bool FCascadeEdPreviewViewportClient::CanCycleWidgetMode() const
{
	// @todo Cascade: Handled manually for now
	return false;
}

FCascadeEdPreviewViewportClient::~FCascadeEdPreviewViewportClient()
{
}

void FCascadeEdPreviewViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	if (!CascadePtr.IsValid())
	{
		return;
	}

	Canvas->Clear( GetPreviewBackgroundColor());

	// Clear out the lines from the previous frame
	CascadePreviewScene.ClearLineBatcher();

	ULineBatchComponent* LineBatcher = CascadePreviewScene.GetLineBatcher();
	CascadePreviewScene.RemoveComponent(LineBatcher);

	const FVector XAxis(1,0,0); 
	const FVector YAxis(0,1,0); 
	const FVector ZAxis(0,0,1);

	if (GetDrawElement(OriginAxis))
	{
		FMatrix ArrowMatrix = FMatrix(XAxis, YAxis, ZAxis, FVector::ZeroVector);
		LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Red, 10.f, 1.0f, SDPG_World);

		ArrowMatrix = FMatrix(YAxis, ZAxis, XAxis, FVector::ZeroVector);
		LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Green, 10.f, 1.0f, SDPG_World);

		ArrowMatrix = FMatrix(ZAxis, XAxis, YAxis, FVector::ZeroVector);
		LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Blue, 10.f, 1.0f, SDPG_World);
	}

	if (GetDrawElement(WireSphere))
	{
		FVector Base(0.f);
		FColor WireColor = FColor::Red;
		const int32 NumRings = 16;
		const float RotatorMultiplier = 360.f / NumRings;
		const int32 NumSides = 32;
		for (int32 i = 0; i < NumRings; i++)
		{
			FVector RotXAxis;
			FVector RotYAxis;
			FVector RotZAxis;

			FRotationMatrix RotMatrix(FRotator(i * RotatorMultiplier, 0, 0));
			RotXAxis = RotMatrix.TransformPosition(XAxis);
			RotYAxis = RotMatrix.TransformPosition(YAxis);
			RotZAxis = RotMatrix.TransformPosition(ZAxis);
			LineBatcher->DrawCircle(Base, RotXAxis, RotYAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);
			LineBatcher->DrawCircle(Base, RotXAxis, RotZAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);
			LineBatcher->DrawCircle(Base, RotYAxis, RotZAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);

			RotMatrix = FRotationMatrix(FRotator(0, i * RotatorMultiplier, 0));
			RotXAxis = RotMatrix.TransformPosition(XAxis);
			RotYAxis = RotMatrix.TransformPosition(YAxis);
			RotZAxis = RotMatrix.TransformPosition(ZAxis);
			LineBatcher->DrawCircle(Base, RotXAxis, RotYAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);
			LineBatcher->DrawCircle(Base, RotXAxis, RotZAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);
			LineBatcher->DrawCircle(Base, RotYAxis, RotZAxis, WireColor, WireSphereRadius, NumSides, SDPG_World);
		}
	}

	FEngineShowFlags SavedEngineShowFlags = EngineShowFlags;

	if (GetDrawElement(Bounds))
	{
		EngineShowFlags.SetBounds(true);
		EngineShowFlags.Game = 1;
	}

	EngineShowFlags.SetVectorFields(GetDrawElement(VectorFields));

	CascadePreviewScene.AddComponent(LineBatcher,FTransform::Identity);

	FVector2D ScaledViewportSize = FVector2D(InViewport->GetSizeXY()) / Canvas->GetDPIScale();
	FEditorViewportClient::Draw(InViewport, Canvas);

	EngineShowFlags = SavedEngineShowFlags;
	FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), GEngine->GetTinyFont(), FLinearColor::White );
	if (GetDrawElement(ParticleCounts) || GetDrawElement(ParticleTimes) || GetDrawElement(ParticleEvents) || GetDrawElement(ParticleMemory) || GetDrawElement(ParticleSystemCompleted))
	{
		// 'Up' from the lower left...
		FString strOutput;
		const int32 XPosition = ScaledViewportSize.X - 5;
		int32 YPosition = ScaledViewportSize.Y - (GetDrawElement(ParticleMemory) ? 15 : 5);

		UParticleSystemComponent* PartComp = CascadePtr.Pin()->GetParticleSystemComponent();

		int32 iWidth, iHeight;

		if (PartComp->EmitterInstances.Num())
		{
			for (int32 i = 0; i < PartComp->EmitterInstances.Num(); i++)
			{
				FParticleEmitterInstance* Instance = PartComp->EmitterInstances[i];
				if (!Instance || !Instance->SpriteTemplate)
				{
					continue;
				}
				UParticleLODLevel* LODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
				if (!LODLevel)
				{
					continue;
				}

				if (GetDrawElement(EmitterTickTimes))
				{
					FString StatLine = Instance->SpriteTemplate->EmitterName.ToString() + " tick: " + FString::Printf(TEXT("%.3f ms"), Instance->LastTickDurationMs);
					Canvas->DrawShadowedString(0, i*16+25, *StatLine, GEngine->GetTinyFont(), FLinearColor::Gray);
				}

				strOutput = TEXT("");
				if (Instance->SpriteTemplate->EmitterRenderMode != ERM_None)
				{
					UParticleLODLevel* HighLODLevel = Instance->SpriteTemplate->GetLODLevel(0);
					if (GetDrawElement(ParticleCounts))
					{
						strOutput += FString::Printf(TEXT("%4d/%4d"), 
							Instance->ActiveParticles, HighLODLevel->PeakActiveParticles);
					}
					if (GetDrawElement(ParticleTimes))
					{
						if (GetDrawElement(ParticleCounts))
						{
							strOutput += TEXT("/");
						}
						strOutput += FString::Printf(TEXT("%8.4f/%8.4f"), 
							Instance->EmitterTime, Instance->SecondsSinceCreation);
					}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (GetDrawElement(ParticleEvents))
					{
						if (GetDrawElement(ParticleCounts) || GetDrawElement(ParticleTimes))
						{
							strOutput += TEXT("/");
						}
						strOutput += FString::Printf(TEXT("Evts: %4d/%4d"), Instance->EventCount, Instance->MaxEventCount);
					}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					UCanvas::ClippedStrLen(GEngine->GetTinyFont(), 1.0f, 1.0f, iWidth, iHeight, *strOutput);
					TextItem.SetColor( Instance->SpriteTemplate->EmitterEditorColor );
					TextItem.Text = FText::FromString( strOutput );
					Canvas->DrawItem( TextItem, XPosition - iWidth, YPosition - iHeight );
					YPosition -= iHeight - 2;
				}
			}

			if (GetDrawElement(ParticleMemory))
			{
				YPosition = ScaledViewportSize.Y - 5;
				FString MemoryOutput = FString::Printf(TEXT("Template: %.0f KByte / Instance: %.0f KByte"), 
					(float)ParticleSystemRootSize / 1024.0f + (float)ParticleModuleMemSize / 1024.0f,
					(float)PSysCompRootSize / 1024.0f + (float)PSysCompResourceSize / 1024.0f);
				UCanvas::ClippedStrLen(GEngine->GetTinyFont(), 1.0f, 1.0f, iWidth, iHeight, *MemoryOutput);
				TextItem.SetColor( FLinearColor::White );
				TextItem.Text = FText::FromString( MemoryOutput );
				Canvas->DrawItem( TextItem, XPosition - iWidth, YPosition - iHeight );				
			}
		}
		else
		{
			for (int32 i = 0; i < PartComp->Template->Emitters.Num(); i++)
			{
				strOutput = TEXT("");
				UParticleEmitter* Emitter = PartComp->Template->Emitters[i];
				UParticleLODLevel* LODLevel = Emitter->GetLODLevel(0);
				if (LODLevel && LODLevel->bEnabled && (Emitter->EmitterRenderMode != ERM_None))
				{
					if (GetDrawElement(ParticleCounts))
					{
						strOutput += FString::Printf(TEXT("%4d/%4d"), 
							0, LODLevel->PeakActiveParticles);
					}
					if (GetDrawElement(ParticleTimes))
					{
						if (GetDrawElement(ParticleCounts))
						{
							strOutput += TEXT("/");
						}
						strOutput += FString::Printf(TEXT("%8.4f/%8.4f"), 0.f, 0.f);
					}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (GetDrawElement(ParticleEvents))
					{
						if (GetDrawElement(ParticleCounts) || GetDrawElement(ParticleTimes))
						{
							strOutput += TEXT("/");
						}
						strOutput += FString::Printf(TEXT("Evts: %4d/%4d"), 0, 0);
					}
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					UCanvas::ClippedStrLen(GEngine->GetTinyFont(), 1.0f, 1.0f, iWidth, iHeight, *strOutput);
					TextItem.SetColor( Emitter->EmitterEditorColor );
					TextItem.Text = FText::FromString( strOutput );
					Canvas->DrawItem( TextItem, XPosition - iWidth, YPosition - iHeight );					
					YPosition -= iHeight - 2;
				}
			}

			if (GetDrawElement(ParticleMemory))
			{
				YPosition = ScaledViewportSize.Y - 5;
				FString MemoryOutput = FString::Printf(TEXT("Template: %.0f KByte / Instance: %.0f KByte"), 
					(float)ParticleSystemRootSize / 1024.0f + (float)ParticleModuleMemSize / 1024.0f,
					(float)PSysCompRootSize / 1024.0f + (float)PSysCompResourceSize / 1024.0f);
				UCanvas::ClippedStrLen(GEngine->GetTinyFont(), 1.0f, 1.0f, iWidth, iHeight, *MemoryOutput);
				TextItem.SetColor( FLinearColor::White );
				TextItem.Text = FText::FromString( MemoryOutput );
				Canvas->DrawItem( TextItem, XPosition - iWidth, YPosition - iHeight );				
			}
		}

		if (GetDrawElement(ParticleSystemCompleted))
		{
			if (PartComp->HasCompleted())
			{
				TextItem.SetColor(FLinearColor::White);
				TextItem.Text = LOCTEXT("SystemCompleted", "Completed");
				TextItem.bCentreX = true;
				TextItem.bCentreY = true;
				Canvas->DrawItem(TextItem, ScaledViewportSize.X * 0.5f, ScaledViewportSize.Y - 10);
				TextItem.bCentreX = false;
				TextItem.bCentreY = false;
			}
		}
	}

	float MiscMessageY = ScaledViewportSize.Y - 75.0f;
	float MisMessageYSep = 15;
	//Display a warning message in the preview window if the system has no fixed bounding-box and contains a GPU emitter.
	if(CascadePtr.Pin()->GetParticleSystem()->bUseFixedRelativeBoundingBox == false)
	{
		UParticleSystemComponent* PartComp = CascadePtr.Pin()->GetParticleSystemComponent();
		if (PartComp->EmitterInstances.Num())
		{
			//We iterate over the emitter instances to find any that contain a GPU Sprite TypeData module. If found, we draw the warning message.
			for (int32 i = 0; i < PartComp->EmitterInstances.Num(); i++)
			{
				FParticleEmitterInstance* Instance = PartComp->EmitterInstances[i];
				if(!Instance || !Instance->SpriteTemplate)
				{
					continue;
				}

				UParticleLODLevel* LODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
				if (!LODLevel || !LODLevel->TypeDataModule)
				{
					continue;
				}

				const bool bIsAGPUEmitter = LODLevel->TypeDataModule->IsA(UParticleModuleTypeDataGpu::StaticClass());
				if(bIsAGPUEmitter)
				{
					const int32 XPosition = 5;
					const int32 YPosition = MiscMessageY;
					MiscMessageY -= MisMessageYSep;
					FString strOutput = NSLOCTEXT("Cascade", "NoFixedBounds_Warning", "WARNING: This particle system has no fixed bounding box and contains a GPU emitter.").ToString();
					TextItem.SetColor( FLinearColor::White );
					TextItem.Text = FText::FromString( strOutput );
					Canvas->DrawItem( TextItem, XPosition, YPosition );					
					break;
				}
			}
		}
	}

	int32 DetailMode = CascadePtr.Pin()->GetDetailMode();
	
	if (DetailMode != DM_Epic)
	{
		FString DetailModeOutput = FString::Printf(TEXT("DETAIL MODE: %s"), (DetailMode == DM_High) ? TEXT("HIGH") : (DetailMode == DM_Medium) ? TEXT("MEDIUM") : TEXT("LOW"));
		TextItem.SetColor( FLinearColor::Red );
		TextItem.Text = FText::FromString( DetailModeOutput );
		Canvas->DrawItem(TextItem, 5.0f, MiscMessageY);
		MiscMessageY -= MisMessageYSep;
	}

	if (GEngine->bEnableEditorPSysRealtimeLOD)
	{
		TextItem.SetColor( FLinearColor(0.25f, 0.25f, 1.0f) );
		TextItem.Text = LOCTEXT("LODPREVIEWMODEENABLED","LOD PREVIEW MODE ENABLED");
		Canvas->DrawItem(TextItem, 5.0f, MiscMessageY);
		MiscMessageY -= MisMessageYSep;
	}

	EParticleSignificanceLevel ReqSignificance = CascadePtr.Pin()->GetRequiredSignificance();
	if (ReqSignificance != EParticleSignificanceLevel::Low)
	{
		FString ReqSigOutput = FString::Printf(TEXT("REQUIRED SIGNIFICANCE: %s"), (ReqSignificance == EParticleSignificanceLevel::Medium) ? TEXT("MEDIUM") : ((ReqSignificance == EParticleSignificanceLevel::High) ? TEXT("HIGH") : TEXT("CRITICAL")));
		TextItem.SetColor(FLinearColor::Red);
		TextItem.Text = FText::FromString(ReqSigOutput);
		Canvas->DrawItem(TextItem, 5.0f, MiscMessageY);
		MiscMessageY -= MisMessageYSep;
	}
	
	if(UParticleSystem* PSys = CascadePtr.Pin()->GetParticleSystem())
	{
		if (!PSys->CanTickInAnyThread())
		{
			//Place a warning on screen if this system cannot tick async.
			TextItem.SetColor(FLinearColor::Red);
			TextItem.Text = FText::FromString(TEXT("NO ASYNC TICK"));
			Canvas->DrawItem(TextItem, 5.0f, MiscMessageY);
			MiscMessageY -= MisMessageYSep;
		}
	}


	if (bCaptureScreenShot)
	{
		UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
		int32 SrcWidth = InViewport->GetSizeXY().X;
		int32 SrcHeight = InViewport->GetSizeXY().Y;
		// Read the contents of the viewport into an array.
		TArray<FColor> OrigBitmap;
		if (InViewport->ReadPixels(OrigBitmap))
		{
			check(OrigBitmap.Num() == SrcWidth * SrcHeight);

			// Resize image to enforce max size.
			TArray<FColor> ScaledBitmap;
			int32 ScaledWidth	 = 512;
			int32 ScaledHeight = 512;
			FImageUtils::CropAndScaleImage(SrcWidth, SrcHeight, ScaledWidth, ScaledHeight, OrigBitmap, ScaledBitmap);

			// Compress.
			FCreateTexture2DParameters Params;
			Params.bDeferCompression = true;
			ParticleSystem->ThumbnailImage = FImageUtils::CreateTexture2D(ScaledWidth, ScaledHeight, ScaledBitmap, ParticleSystem, TEXT("ThumbnailTexture"), RF_NoFlags, Params);

			ParticleSystem->ThumbnailImageOutOfDate = false;
			ParticleSystem->MarkPackageDirty();
		}

		bCaptureScreenShot = false;
	}
}

void FCascadeEdPreviewViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	DrawHelper.Draw(View, PDI);

	// If a local vector field module is selected, draw a widget so that the user can move the vector field around.
	UParticleModuleVectorFieldLocal* VectorFieldModule = Cast<UParticleModuleVectorFieldLocal>(CascadePtr.Pin()->GetSelectedModule());
	if (VectorFieldModule)
	{
		const FVector WidgetOrigin = VectorFieldModule->RelativeTranslation;
		const FRotator WidgetRotation = (WidgetMM == WMM_Translate) ? FRotator::ZeroRotator : VectorFieldModule->RelativeRotation;
		const FTransform WidgetTransform(
			WidgetRotation,
			WidgetOrigin,
			FVector(1.0f,1.0f,1.0f));
		FUnrealEdUtils::DrawWidget(View, PDI, WidgetTransform.ToMatrixWithScale(), VectorFieldHitproxyInfo, 0, WidgetAxis, WidgetMM);
	}

	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	UCascadeParticleSystemComponent* ParticleSystemComponent = CascadePtr.Pin()->GetParticleSystemComponent();
	// Can now iterate over the modules on this system...
	for (int32 i = 0; i < ParticleSystem->Emitters.Num(); i++)
	{
		UParticleEmitter* Emitter = ParticleSystem->Emitters[i];
		if (Emitter == NULL)
		{
			continue;
		}

		// Emitters may have a set number of loops.
		// After which, the system will kill them off
		if (i < ParticleSystemComponent->EmitterInstances.Num())
		{
			FParticleEmitterInstance* EmitterInst = ParticleSystemComponent->EmitterInstances[i];
			if (EmitterInst && EmitterInst->SpriteTemplate)
			{
				check(EmitterInst->SpriteTemplate == Emitter);

				UParticleLODLevel* LODLevel = Emitter->GetCurrentLODLevel(EmitterInst);
				for(int32 j=0; j< LODLevel->Modules.Num(); j++)
				{
					UParticleModule* Module = LODLevel->Modules[j];
					if (Module && Module->bSupported3DDrawMode && Module->b3DDrawMode)
					{
						Module->Render3DPreview({ *EmitterInst, View, PDI });
					}
				}
			}
		}
	}

	// Draw the preview scene light visualization
	DrawPreviewLightVisualization(View, PDI);
}

bool FCascadeEdPreviewViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	//Update cursor 
	UpdateAndApplyCursorVisibility();

	bool bHandled = false;
	const int32 HitX = InEventArgs.Viewport->GetMouseX();
	const int32 HitY = InEventArgs.Viewport->GetMouseY();

	if(InEventArgs.Key == EKeys::LeftMouseButton)
	{
		if (InEventArgs.Event == IE_Pressed)
		{
			InEventArgs.Viewport->InvalidateHitProxy();
			HHitProxy* HitResult = InEventArgs.Viewport->GetHitProxy(HitX,HitY);
			if (HitResult && HitResult->IsA(HWidgetUtilProxy::StaticGetType()))
			{
				HWidgetUtilProxy* WidgetProxy = (HWidgetUtilProxy*)HitResult;
				if (WidgetProxy->Info1 == VectorFieldHitproxyInfo)
				{
					bManipulatingVectorField = true;
				}
				WidgetAxis = WidgetProxy->Axis;

				// Calculate the scree-space directions for this drag.
				FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( InEventArgs.Viewport, GetScene(), EngineShowFlags ));
				FSceneView* View = CalcSceneView(&ViewFamily);
				WidgetProxy->CalcVectors(View, FViewportClick(View, this, InEventArgs.Key, InEventArgs.Event, HitX, HitY), LocalManipulateDir, WorldManipulateDir, DragX, DragY);
				bHandled = true;
			}
		}
		else if (InEventArgs.Event == IE_Released)
		{
			if (bManipulatingVectorField)
			{
				WidgetAxis = EAxisList::None;
				bManipulatingVectorField = false;

				bHandled = true;
			}
		}
	}
	else if (InEventArgs.Key == EKeys::SpaceBar && InEventArgs.Event == IE_Pressed)
	{
		if (CascadePtr.Pin()->GetSelectedModule() && CascadePtr.Pin()->GetSelectedModule()->IsA(UParticleModuleVectorFieldLocal::StaticClass()))
		{
			bHandled = true;

			WidgetMM = (EWidgetMovementMode)((WidgetMM+1) % WMM_MAX);
		}
	}

	if( !bHandled )
	{
		bHandled = FEditorViewportClient::InputKey(InEventArgs);
	}


	return bHandled;
}

// Tweakable speeds for manipulating the widget.
//static TAutoConsoleVariable<float> CVarCascadeDragSpeed(TEXT("CascadeDragSpeed"),1.0f,TEXT("Cascade drag speed."));
static auto CVarCascadeDragSpeed = TAutoConsoleVariable<float>(TEXT("CascadeDragSpeed"),1.0f,TEXT("Cascade drag speed."));
static TAutoConsoleVariable<float> CVarCascadeRotateSpeed(TEXT("CascadeRotateSpeed"),0.005f,TEXT("Cascade drag speed."));
static TAutoConsoleVariable<float> CVarCascadeScaleSpeed(TEXT("CascadeScaleSpeed"),1.0f,TEXT("Cascade scale speed."));

bool FCascadeEdPreviewViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	bool bHandled = false;

	const float Delta = Args.AmountDepressed;

	if (bManipulatingVectorField)
	{
		UParticleModuleVectorFieldLocal* VectorFieldModule = Cast<UParticleModuleVectorFieldLocal>(CascadePtr.Pin()->GetSelectedModule());
		if (VectorFieldModule)
		{
			const float MoveX = ((Args.Key == EKeys::MouseX) ? Delta : 0.0f) * DragX;
			const float MoveY = ((Args.Key == EKeys::MouseY) ? Delta : 0.0f) * DragY;
			const float MoveAmount = MoveX + MoveY;

			VectorFieldModule->PreEditChange(nullptr);
			if (WidgetMM == WMM_Translate)
			{
				VectorFieldModule->RelativeTranslation += (LocalManipulateDir * MoveAmount * CVarCascadeDragSpeed.GetValueOnGameThread());
			}
			else if (WidgetMM == WMM_Rotate)
			{
				const FQuat CurrentRotation = VectorFieldModule->RelativeRotation.Quaternion();
				const FQuat DeltaRotation(LocalManipulateDir, -MoveAmount * CVarCascadeRotateSpeed.GetValueOnGameThread());
				const FQuat NewRotation = CurrentRotation * DeltaRotation;
				VectorFieldModule->RelativeRotation = FRotator(NewRotation);
			}
			else if (WidgetMM == WMM_Scale)
			{
				VectorFieldModule->RelativeScale3D += (LocalManipulateDir * MoveAmount * CVarCascadeScaleSpeed.GetValueOnGameThread());
			}
			VectorFieldModule->PostEditChange();
		}

		bHandled = true;
	}
	else
	{
		bHandled = FEditorViewportClient::InputAxis(Args);
	}

	if (!IsRealtime() && !FMath::IsNearlyZero(Delta))
	{
		Args.Viewport->Invalidate();
	}

	return bHandled;
}

void FCascadeEdPreviewViewportClient::SetPreviewCamera(const FRotator& NewPreviewAngle, float NewPreviewDistance)
{
	PreviewAngle = NewPreviewAngle;
	PreviewDistance = NewPreviewDistance;

	SetViewLocation( PreviewAngle.Vector() * -PreviewDistance );
	SetViewRotation( PreviewAngle );

	Viewport->Invalidate();
}

void FCascadeEdPreviewViewportClient::UpdateMemoryInformation()
{
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	UCascadeParticleSystemComponent* ParticleSystemComponent = CascadePtr.Pin()->GetParticleSystemComponent();
	if (ParticleSystem != NULL)
	{
		FArchiveCountMem MemCount(ParticleSystem);
		ParticleSystemRootSize = MemCount.GetMax();

		ParticleModuleMemSize = 0;
		TMap<UParticleModule*,bool> ModuleList;
		for (int32 EmitterIdx = 0; EmitterIdx < ParticleSystem->Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = ParticleSystem->Emitters[EmitterIdx];
			if (Emitter != NULL)
			{
				for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
					if (LODLevel != NULL)
					{
						ModuleList.Add(LODLevel->RequiredModule, true);
						ModuleList.Add(LODLevel->SpawnModule, true);
						for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							ModuleList.Add(LODLevel->Modules[ModuleIdx], true);
						}
					}
				}
			}
		}
		for (TMap<UParticleModule*,bool>::TIterator ModuleIt(ModuleList); ModuleIt; ++ModuleIt)
		{
			UParticleModule* Module = ModuleIt.Key();
			FArchiveCountMem ModuleCount(Module);
			ParticleModuleMemSize += ModuleCount.GetMax();
		}
	}
	if (ParticleSystemComponent != NULL)
	{
		FArchiveCountMem ComponentMemCount(ParticleSystemComponent);
		PSysCompRootSize = ComponentMemCount.GetMax();
		PSysCompResourceSize = ParticleSystemComponent->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
	}
}

void FCascadeEdPreviewViewportClient::CreateThumbnail()
{
	UParticleSystem* ParticleSystem = CascadePtr.Pin()->GetParticleSystem();
	
	ParticleSystem->ThumbnailAngle = PreviewAngle;
	ParticleSystem->ThumbnailDistance = PreviewDistance;
	ParticleSystem->PreviewComponent = NULL;

	bCaptureScreenShot = true;
}

FSceneInterface* FCascadeEdPreviewViewportClient::GetScene() const
{
	return CascadePreviewScene.GetScene();
}

FLinearColor FCascadeEdPreviewViewportClient::GetBackgroundColor() const
{
	return GetPreviewBackgroundColor();
}

bool FCascadeEdPreviewViewportClient::ShouldOrbitCamera() const
{
	if (GetDefault<ULevelEditorViewportSettings>()->bUseUE3OrbitControls)
	{
		// this editor orbits always if ue3 orbit controls are enabled
		return true;
	}

	return FEditorViewportClient::ShouldOrbitCamera();
}

FPreviewScene& FCascadeEdPreviewViewportClient::GetPreviewScene()
{
	return CascadePreviewScene;
}

bool FCascadeEdPreviewViewportClient::GetDrawElement(EDrawElements Element) const
{
	return (DrawFlags & Element) != 0;
}

void FCascadeEdPreviewViewportClient::ToggleDrawElement(EDrawElements Element)
{
	DrawFlags = DrawFlags ^ Element;
}

FColor FCascadeEdPreviewViewportClient::GetPreviewBackgroundColor() const
{
	if (CascadePtr.IsValid() && CascadePtr.Pin()->GetParticleSystem())
	{
		return  CascadePtr.Pin()->GetParticleSystem()->BackgroundColor;
	}

	return BackgroundColor;
}

UStaticMeshComponent* FCascadeEdPreviewViewportClient::GetFloorComponent()
{
	return FloorComponent;
}

FEditorCommonDrawHelper& FCascadeEdPreviewViewportClient::GetDrawHelper()
{
	return DrawHelper;
}

float& FCascadeEdPreviewViewportClient::GetWireSphereRadius()
{
	return WireSphereRadius;
}
#undef LOCTEXT_NAMESPACE 
