// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "InputCoreTypes.h"
#include "Math/Axis.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PreviewScene.h"
#include "Templates/SharedPointer.h"
#include "Utils.h"

class FCanvas;
class FCascade;
class FEditorCommonDrawHelper;
class FPrimitiveDrawInterface;
class FReferenceCollector;
class FSceneInterface;
class FSceneView;
class FViewport;
class SCascadePreviewViewport;
class UStaticMeshComponent;
struct FInputKeyEventArgs;

/*-----------------------------------------------------------------------------
   FCascadeViewportClient
-----------------------------------------------------------------------------*/

class FCascadeEdPreviewViewportClient : public FEditorViewportClient
{
public:
	/** Constructor */
	FCascadeEdPreviewViewportClient(TWeakPtr<FCascade> InCascade, const TSharedRef<SCascadePreviewViewport>& InCascadeViewport);
	virtual ~FCascadeEdPreviewViewportClient() override;

	/** FEditorViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& InEventArgs) override;
	virtual FSceneInterface* GetScene() const override;
	virtual FLinearColor GetBackgroundColor() const override;
	virtual bool ShouldOrbitCamera() const override;
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual bool CanCycleWidgetMode() const override;

	/** Sets the position and orientation of the preview camera */
	void SetPreviewCamera(const FRotator& NewPreviewAngle, float NewPreviewDistance);

	/** Update the memory information of the particle system */
	void UpdateMemoryInformation();

	/** Generates a new thumbnail image for the content browser */
	void CreateThumbnail();

	/** Draw flag types */
	enum EDrawElements
	{
		ParticleCounts = 0x001,
		ParticleEvents = 0x002,
		ParticleTimes = 0x004,
		ParticleMemory = 0x008,
		VectorFields = 0x010,
		Bounds = 0x020,
		WireSphere = 0x040,
		OriginAxis = 0x080,
		Orbit = 0x100,
		ParticleSystemCompleted = 0x200,
		EmitterTickTimes = 0x400
	};

	/** Accessors */
	FPreviewScene& GetPreviewScene();
	bool GetDrawElement(EDrawElements Element) const;
	void ToggleDrawElement(EDrawElements Element);
	FColor GetPreviewBackgroundColor() const;
	UStaticMeshComponent* GetFloorComponent();
	FEditorCommonDrawHelper& GetDrawHelper();
	float& GetWireSphereRadius();

	FCascade* GetCascade(){ return CascadePtr.IsValid() ? CascadePtr.Pin().Get() : nullptr; }
private:
	/** Pointer back to the ParticleSystem editor tool that owns us */
	TWeakPtr<FCascade> CascadePtr;
	
	/** Preview mesh */
	UStaticMeshComponent* FloorComponent;

	/** Camera potition/rotation */
	FRotator PreviewAngle;
	float PreviewDistance;

	/** If true, will take screenshot for thumbnail on next draw call */
	float bCaptureScreenShot;

	/** User input state info */
	FVector WorldManipulateDir;
	FVector LocalManipulateDir;
	float DragX;
	float DragY;
	EAxisList::Type WidgetAxis;
	EWidgetMovementMode WidgetMM;
	bool bManipulatingVectorField;

	/** Draw flags (see EDrawElements) */
	int32 DrawFlags;

	/** Radius of the wireframe sphere */
	float WireSphereRadius;

	/** Veiwport background color */
	FColor BackgroundColor;
	
	/** The scene used for the viewport. Owned externally */
	FPreviewScene CascadePreviewScene;

	/** The size of the ParticleSystem via FArchive memory counting */
	int32 ParticleSystemRootSize;
	/** The size the particle modules take for the system */
	int32 ParticleModuleMemSize;
	/** The size of the ParticleSystemComponent via FArchive memory counting */
	int32 PSysCompRootSize;
	/** The size of the ParticleSystemComponent resource size */
	int32 PSysCompResourceSize;

	/** Draw info index for vector fields */
	const int32 VectorFieldHitproxyInfo;

	/** Speed multiplier used when moving the scene light around */
	const float LightRotSpeed;
};

