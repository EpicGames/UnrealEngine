// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCascadeEmitterCanvas.h"

#include "CascadeEmitterCanvasClient.h"
#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/Children.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"

void SCascadeEmitterCanvas::Construct(const FArguments& InArgs)
{
	CascadePtr = InArgs._Cascade;
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					SAssignNew(ViewportWidget, SViewport)
					.EnableGammaCorrection(false)
					.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
					.ShowEffectWhenDisabled(false)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ViewportVerticalScrollBar, SScrollBar)
				.Visibility(this, &SCascadeEmitterCanvas::GetViewportVerticalScrollBarVisibility)
				.OnUserScrolled(this, &SCascadeEmitterCanvas::OnViewportVerticalScrollBarScrolled)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ViewportHorizontalScrollBar, SScrollBar)
			.Orientation( Orient_Horizontal )
			.Visibility(this, &SCascadeEmitterCanvas::GetViewportHorizontalScrollBarVisibility)
			.OnUserScrolled(this, &SCascadeEmitterCanvas::OnViewportHorizontalScrollBarScrolled)
		]
	];
	
	ViewportClient = MakeShareable(new FCascadeEmitterCanvasClient(CascadePtr.Pin(), SharedThis(this)));

	ViewportClient->bSetListenerPosition = false;

	ViewportClient->VisibilityDelegate.BindSP(this, &SCascadeEmitterCanvas::IsVisible);

	Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);
}

void SCascadeEmitterCanvas::RefreshViewport()
{
	Viewport->Invalidate();
	Viewport->InvalidateDisplay();
}

bool SCascadeEmitterCanvas::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground());
}

TSharedPtr<FSceneViewport> SCascadeEmitterCanvas::GetViewport() const
{
	return Viewport;
}

TSharedPtr<FCascadeEmitterCanvasClient> SCascadeEmitterCanvas::GetViewportClient() const
{
	return ViewportClient;
}

TSharedPtr<SViewport> SCascadeEmitterCanvas::GetViewportWidget() const
{
	return ViewportWidget;
}

TSharedPtr<SScrollBar> SCascadeEmitterCanvas::GetVerticalScrollBar() const
{
	return ViewportVerticalScrollBar;
}

TSharedPtr<SScrollBar> SCascadeEmitterCanvas::GetHorizontalScrollBar() const
{
	return ViewportHorizontalScrollBar;
}

EVisibility SCascadeEmitterCanvas::GetViewportVerticalScrollBarVisibility() const
{
	return (ViewportClient->GetViewportVerticalScrollBarRatio() < 1.0f)? EVisibility::Visible: EVisibility::Collapsed;
}

EVisibility SCascadeEmitterCanvas::GetViewportHorizontalScrollBarVisibility() const
{
	return (ViewportClient->GetViewportHorizontalScrollBarRatio() < 1.0f)? EVisibility::Visible: EVisibility::Collapsed;
}

void SCascadeEmitterCanvas::OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction)
{
	float Ratio = ViewportClient->GetViewportVerticalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);
	ViewportVerticalScrollBar->SetState(InScrollOffsetFraction, Ratio);
	RefreshViewport();
}

void SCascadeEmitterCanvas::OnViewportHorizontalScrollBarScrolled(float InScrollOffsetFraction)
{
	float Ratio = ViewportClient->GetViewportHorizontalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);
	ViewportHorizontalScrollBar->SetState(InScrollOffsetFraction, Ratio);
	RefreshViewport();
}
