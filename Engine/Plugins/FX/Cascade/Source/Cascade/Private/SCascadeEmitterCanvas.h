// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FCascade;
class FCascadeEmitterCanvasClient;
class FSceneViewport;
class SDockTab;
class SScrollBar;
class SViewport;

/*-----------------------------------------------------------------------------
   SCascadeViewport
-----------------------------------------------------------------------------*/

class SCascadeEmitterCanvas : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCascadeEmitterCanvas)
		{}

		SLATE_ARGUMENT(TWeakPtr<FCascade>, Cascade)
	SLATE_END_ARGS()

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);
	
	/** Refreshes the viewport */
	void RefreshViewport();

	/** Returns true if the viewport is visible */
	bool IsVisible() const;

	/** Accessors */
	TSharedPtr<FSceneViewport> GetViewport() const;
	TSharedPtr<FCascadeEmitterCanvasClient> GetViewportClient() const;
	TSharedPtr<SViewport> GetViewportWidget() const;
	TSharedPtr<SScrollBar> GetVerticalScrollBar() const;
	TSharedPtr<SScrollBar> GetHorizontalScrollBar() const;

public:
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

protected:
	/** Returns the visibility of the viewport scrollbars */
	EVisibility GetViewportVerticalScrollBarVisibility() const;
	EVisibility GetViewportHorizontalScrollBarVisibility() const;

	/** Called when the viewport scrollbars are scrolled */
	void OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction);
	void OnViewportHorizontalScrollBarScrolled(float InScrollOffsetFraction);

private:
	/** Pointer back to the Texture editor tool that owns us */
	TWeakPtr<FCascade> CascadePtr;
	
	/** Level viewport client */
	TSharedPtr<FCascadeEmitterCanvasClient> ViewportClient;

	/** Slate viewport for rendering and I/O */
	TSharedPtr<FSceneViewport> Viewport;

	/** Viewport widget*/
	TSharedPtr<SViewport> ViewportWidget;

	/** Vertical scrollbar */
	TSharedPtr<SScrollBar> ViewportVerticalScrollBar;

	/** Horizontal scrollbar */
	TSharedPtr<SScrollBar> ViewportHorizontalScrollBar;
};
