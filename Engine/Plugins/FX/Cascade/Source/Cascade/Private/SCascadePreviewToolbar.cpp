// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCascadePreviewToolbar.h"

#include "Cascade.h"
#include "CascadeActions.h"
#include "CascadePreviewViewportClient.h"
#include "Delegates/Delegate.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SCascadePreviewViewport.h"
#include "SEditorViewportToolBarMenu.h"
#include "SViewportToolBar.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

class FTagMetaData;
class SWidget;


void SCascadePreviewViewportToolBar::Construct(const FArguments& InArgs)
{
	CascadePtr = InArgs._CascadePtr;
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 2.0f)
			[
				SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.Cursor(EMouseCursor::Default)
				.Label(NSLOCTEXT("Cascade", "ViewMenuTitle_Default", "View"))
				.OnGetMenuContent(this, &SCascadePreviewViewportToolBar::GenerateViewMenu) 
				.AddMetaData<FTagMetaData>(TEXT("CascadeViewButton.View"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 2.0f)
			[
				SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.Cursor(EMouseCursor::Default)
				.Label(NSLOCTEXT("Cascade", "TimeMenuTitle_Default", "Time"))
				.OnGetMenuContent(this, &SCascadePreviewViewportToolBar::GenerateTimeMenu) 
				.AddMetaData<FTagMetaData>(TEXT("CascadeViewButton.Time"))
			]
		]
	];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

TSharedRef<SWidget> SCascadePreviewViewportToolBar::GenerateViewMenu() const
{
	const FCascadeCommands& Actions = FCascadeCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ViewMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CascadePtr.Pin()->GetToolkitCommands());

	struct Local
	{
		static void BuildViewOptionsMenu(FMenuBuilder& Menu)
		{
			Menu.BeginSection("CascadeViewOverlays", NSLOCTEXT("Cascade", "ViewOptionsHeader", "View Overlays"));
			{
				Menu.AddMenuEntry(FCascadeCommands::Get().View_ParticleCounts);
				Menu.AddMenuEntry(FCascadeCommands::Get().View_ParticleEventCounts);
				Menu.AddMenuEntry(FCascadeCommands::Get().View_ParticleTimes);
				Menu.AddMenuEntry(FCascadeCommands::Get().View_ParticleMemory);
				Menu.AddMenuEntry(FCascadeCommands::Get().View_SystemCompleted);
				Menu.AddMenuEntry(FCascadeCommands::Get().View_EmitterTickTimes);
			}
			Menu.EndSection();
		}

		static void BuildViewModesMenu(FMenuBuilder& Menu)
		{
			Menu.BeginSection("CascadeViewMode", NSLOCTEXT("Cascade", "ViewModeHeader", "View Mode"));
			{
				Menu.AddMenuEntry(FCascadeCommands::Get().ViewMode_Wireframe);
				Menu.AddMenuEntry(FCascadeCommands::Get().ViewMode_Unlit);
				Menu.AddMenuEntry(FCascadeCommands::Get().ViewMode_Lit);
				Menu.AddMenuEntry(FCascadeCommands::Get().ViewMode_ShaderComplexity);
			}
			Menu.EndSection();
		}

		static void BuildDetailModesMenu(FMenuBuilder& Menu)
		{
			Menu.BeginSection("CascadeDetailMode", NSLOCTEXT("Cascade", "DetailModeHeader", "Detail Mode"));
			{
				Menu.AddMenuEntry(FCascadeCommands::Get().DetailMode_Low);
				Menu.AddMenuEntry(FCascadeCommands::Get().DetailMode_Medium);
				Menu.AddMenuEntry(FCascadeCommands::Get().DetailMode_High);
				Menu.AddMenuEntry(FCascadeCommands::Get().DetailMode_Epic);
			}
			Menu.EndSection();
		}

		static void BuildSignificanceMenu(FMenuBuilder& Menu)
		{
			Menu.BeginSection("CascadeSignificance", NSLOCTEXT("Cascade", "SignificanceHeader", "Required Significance"));
			{
				Menu.AddMenuEntry(FCascadeCommands::Get().Significance_Critical);
				Menu.AddMenuEntry(FCascadeCommands::Get().Significance_High);
				Menu.AddMenuEntry(FCascadeCommands::Get().Significance_Medium);
				Menu.AddMenuEntry(FCascadeCommands::Get().Significance_Low);
			}
			Menu.EndSection();
		}
	};

	ViewMenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "ViewOptionsSubMenu", "View Overlays"), 
								FText::GetEmpty(), 
								FNewMenuDelegate::CreateStatic(&Local::BuildViewOptionsMenu));

	ViewMenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "ViewModesSubMenu", "View Modes"), 
								FText::GetEmpty(), 
								FNewMenuDelegate::CreateStatic(&Local::BuildViewModesMenu));

	ViewMenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "DetailModesSubMenu", "Detail Modes"), 
								FText::GetEmpty(), 
								FNewMenuDelegate::CreateStatic(&Local::BuildDetailModesMenu));

	ViewMenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "SignificanceSubMenu", "Significance"),
								FText::GetEmpty(),
								FNewMenuDelegate::CreateStatic(&Local::BuildSignificanceMenu));

	ViewMenuBuilder.BeginSection("CascadeMiscPreview");
	{
		// Only display the orbit mode option if orbit cam controls are disabled
		if (!CascadePtr.Pin()->GetPreviewViewport()->GetViewportClient()->bUsingOrbitCamera )
		{
			ViewMenuBuilder.AddMenuEntry(Actions.ToggleOrbitMode);
		}

		ViewMenuBuilder.AddMenuEntry(Actions.ToggleLocalVectorFields);
		ViewMenuBuilder.AddMenuEntry(Actions.ToggleGrid);
		ViewMenuBuilder.AddMenuEntry(Actions.ToggleWireframeSphere);
		ViewMenuBuilder.AddMenuEntry(Actions.TogglePostProcess);
		ViewMenuBuilder.AddMenuEntry(Actions.ToggleMotion);
		ViewMenuBuilder.AddMenuEntry(Actions.SetMotionRadius);
	}
	ViewMenuBuilder.EndSection();

	ViewMenuBuilder.BeginSection("CascadeMiscPreview2");
	{
		ViewMenuBuilder.AddMenuEntry(Actions.ToggleGeometry);
		ViewMenuBuilder.AddMenuEntry(Actions.ToggleGeometry_Properties);
	}
	ViewMenuBuilder.EndSection();

	return ViewMenuBuilder.MakeWidget();	
}

TSharedRef<SWidget> SCascadePreviewViewportToolBar::GenerateTimeMenu() const
{
	const FCascadeCommands& Actions = FCascadeCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder TimeMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CascadePtr.Pin()->GetToolkitCommands());

	struct Local
	{
		static void BuildAnimSpeedOptionsMenu(FMenuBuilder& Menu)
		{
			Menu.BeginSection("CascadeAnimSpeed", NSLOCTEXT("Cascade", "AnimSpeedHeader", "AnimSpeed"));
			{
				Menu.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_100);
				Menu.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_50);
				Menu.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_25);
				Menu.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_10);
				Menu.AddMenuEntry(FCascadeCommands::Get().AnimSpeed_1);
			}
			Menu.EndSection();
		}
	};

	TimeMenuBuilder.BeginSection("CascadeTimeMenu");
	{
		TimeMenuBuilder.AddMenuEntry(FCascadeCommands::Get().CascadePlay);
	}
	TimeMenuBuilder.EndSection();

	TimeMenuBuilder.BeginSection("CascadeTimeMenu2");
	{
		TimeMenuBuilder.AddMenuEntry(FCascadeCommands::Get().ToggleRealtime);
		TimeMenuBuilder.AddMenuEntry(FCascadeCommands::Get().ToggleLoopSystem);
	}
	TimeMenuBuilder.EndSection();

	TimeMenuBuilder.AddSubMenu(NSLOCTEXT("Cascade", "AnimSpeedSubMenu", "AnimSpeed"), 
		FText::GetEmpty(), 
		FNewMenuDelegate::CreateStatic(&Local::BuildAnimSpeedOptionsMenu));

	return TimeMenuBuilder.MakeWidget();	
}
