// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/*-----------------------------------------------------------------------------
   SCascadePreviewViewportToolBar
-----------------------------------------------------------------------------*/

#include "SViewportToolBar.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FCascade;
class SWidget;

class SCascadePreviewViewportToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SCascadePreviewViewportToolBar){}
		SLATE_ARGUMENT(TWeakPtr<FCascade>, CascadePtr)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Generates the toolbar view menu content */
	TSharedRef<SWidget> GenerateViewMenu() const;

	/** Generates the toolbar time menu content */
	TSharedRef<SWidget> GenerateTimeMenu() const;

private:
	/** The viewport that we are in */
	TWeakPtr<FCascade> CascadePtr;
};
