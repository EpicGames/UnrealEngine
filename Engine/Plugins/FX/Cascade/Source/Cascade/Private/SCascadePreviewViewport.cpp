// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCascadePreviewViewport.h"

#include "Cascade.h"
#include "CascadeParticleSystemComponent.h"
#include "CascadePreviewViewportClient.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/BoxSphereBounds.h"
#include "Math/UnrealMathSSE.h"
#include "Particles/ParticleSystemComponent.h"
#include "PreviewScene.h"
#include "RHIDefinitions.h"
#include "SCascadePreviewToolbar.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SOverlay.h"

class SWidget;
struct FGeometry;



void SCascadePreviewViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	JustTicked = true;
}


SCascadePreviewViewport::~SCascadePreviewViewport()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	Editor->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
}

void SCascadePreviewViewport::Construct(const FArguments& InArgs)
{
	CascadePtr = InArgs._Cascade;

	SEditorViewport::Construct( SEditorViewport::FArguments() );

	// Restore last used feature level
	if (ViewportClient.IsValid())
	{
		UWorld* World = ViewportClient->GetPreviewScene().GetWorld();
		if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
		{
			World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
		}
	}

	// Use a delegate to inform the attached world of preview platform changes.
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			if(ViewportClient.IsValid())
			{
				UWorld* World = ViewportClient->GetPreviewScene().GetWorld();
				if (World != nullptr)
				{
					World->ShaderPlatformChanged(NewShaderPlatform);
				}
			}
		});
}

void SCascadePreviewViewport::RefreshViewport()
{
	SceneViewport->Invalidate();
}

bool SCascadePreviewViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible();
}

TSharedPtr<FSceneViewport> SCascadePreviewViewport::GetViewport() const
{
	return SceneViewport;
}

TSharedPtr<FCascadeEdPreviewViewportClient> SCascadePreviewViewport::GetViewportClient() const
{
	return ViewportClient;
}

TSharedPtr<SViewport> SCascadePreviewViewport::GetViewportWidget() const
{
	return ViewportWidget;
}

TSharedRef<FEditorViewportClient> SCascadePreviewViewport::MakeEditorViewportClient()
{
	ViewportClient = MakeShareable(new FCascadeEdPreviewViewportClient(CascadePtr, SharedThis(this)));

	ViewportClient->bSetListenerPosition = false;

	ViewportClient->SetRealtime(true);
	ViewportClient->VisibilityDelegate.BindSP(this, &SCascadePreviewViewport::IsVisible);

	return ViewportClient.ToSharedRef();
}

void SCascadePreviewViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);
	
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		[
			SNew(SCascadePreviewViewportToolBar)
			.CascadePtr(CascadePtr)
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		];

	// this widget will display the current viewed feature level
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

void SCascadePreviewViewport::OnFocusViewportToSelection()
{
	UParticleSystemComponent* Component = CascadePtr.Pin()->GetParticleSystemComponent();

	if( Component )
	{
		ViewportClient->FocusViewportOnBox( Component->Bounds.GetBox() );
	}
}
