// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "SEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FCascade;
class FCascadeEdPreviewViewportClient;
class FEditorViewportClient;
class FSceneViewport;
class SDockTab;
class SOverlay;
class SViewport;
class SWidget;
struct FGeometry;

/*-----------------------------------------------------------------------------
   SCascadeViewport
-----------------------------------------------------------------------------*/

class SCascadePreviewViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SCascadePreviewViewport)
		{}

		SLATE_ARGUMENT(TWeakPtr<FCascade>, Cascade)
	SLATE_END_ARGS()

	/** Destructor */
	~SCascadePreviewViewport();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

	/** Refreshes the viewport */
	void RefreshViewport();

	/** Returns true if the viewport is visible */
	bool IsVisible() const override;

	/** Accessors */
	TSharedPtr<FSceneViewport> GetViewport() const;
	TSharedPtr<FCascadeEdPreviewViewportClient> GetViewportClient() const;
	TSharedPtr<SViewport> GetViewportWidget() const;

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	bool HasJustTicked()	{ return JustTicked; }
	void ClearTickFlag()	{ JustTicked = false; }
protected:
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	virtual void OnFocusViewportToSelection() override;
public:
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

private:
	/** Pointer back to the ParticleSystem editor tool that owns us */
	TWeakPtr<FCascade> CascadePtr;
	
	/** Level viewport client */
	TSharedPtr<FCascadeEdPreviewViewportClient> ViewportClient;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	bool JustTicked;
};
