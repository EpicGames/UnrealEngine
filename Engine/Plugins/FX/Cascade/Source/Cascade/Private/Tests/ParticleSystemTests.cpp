// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Framework/Commands/InputChord.h"
#include "Factories/ParticleSystemFactoryNew.h"
#include "Particles/ParticleSystem.h"
#include "Distributions/DistributionVectorUniform.h"


//Automation
#include "Tests/AutomationTestSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Tests/AutomationEditorPromotionCommon.h"

//Particle system
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/Size/ParticleModuleSize.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"


#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "ParticleEditorPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogParticleEditorPromotionTests, Log, All);

/**
* Helper functions used by the build promotion automation test
*/
namespace ParticleEditorPromotionTestUtils
{
	/**
	* Gets saved settings for Particle Editor promotion
	*/
	static FParticleEditorPromotionSettings TestSettings()
	{
		UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		check(AutomationTestSettings);

		return AutomationTestSettings->ParticleEditorPromotionTest;
	}

	/**
	* Imports an asset using the supplied factory, class, and name
	*
	* @param CreateFactory - The factory to use to create the asset
	* @param AssetClass - The class of the new asset
	* @param AssetName - The name to use for the new asset
	*/
	static UObject* CreateAsset(UFactory* CreateFactory, UClass* AssetClass, const FString& AssetName)
	{
		FString PackageName = FString::Printf(TEXT("%s/%s"), *FEditorPromotionTestUtilities::GetGamePath(), *AssetName);
		UPackage* AssetPackage = CreatePackage( *PackageName);
		EObjectFlags Flags = RF_Public | RF_Standalone;

		UObject* CreatedAsset = CreateFactory->FactoryCreateNew(AssetClass, AssetPackage, FName(*AssetName), Flags, NULL, GWarn);

		if (CreatedAsset)
		{
			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(CreatedAsset);

			// Mark the package dirty...
			AssetPackage->MarkPackageDirty();

			UE_LOGF(LogParticleEditorPromotionTests, Display, "Created asset %ls (%ls)", *AssetName, *AssetClass->GetName());
		}
		else
		{
			UE_LOGF(LogParticleEditorPromotionTests, Error, "Unable to create asset of type %ls", *AssetClass->GetName());
		}

		return CreatedAsset;
	}

	// @TODO - Replace with direct Cascade iteraction if possible
	/**
	* Sends the AssetEditor->SaveAsset UI command
	*/
	static void SendSaveCascadeCommand()
	{
		const FString Context = TEXT("AssetEditor");
		const FString Command = TEXT("SaveAsset");

		FInputChord CurrentSaveChord = FEditorPromotionTestUtilities::GetOrSetUICommand(Context, Command);

		const FName FocusWidgetType(TEXT("SCascadeEmitterCanvas"));
		FEditorPromotionTestUtilities::SendCommandToCurrentEditor(CurrentSaveChord, FocusWidgetType);
	}
}

/**
* Container for items related to the create asset test
*/
namespace ParticleEditorPromotionTestHelper
{
	struct FParticleEditorPromotionTest
	{
		/** Pointer to the execution info of this test */
		FAutomationTestExecutionInfo* TestExecutionInfo;

		/** Function definition for the test stage functions */
		typedef bool(ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest::*TestStageFn)();

		/** List of test stage functions */
		TArray<TestStageFn> TestStages;

		/** The index of the test stage we are on */
		int32 CurrentStage;

		// Test assets

		/** Particle created from the "Creating a Particle" stage */
		UParticleSystem* CreatedPS;

		/**
		* Constructor
		*/
		FParticleEditorPromotionTest(FAutomationTestExecutionInfo* InExecutionInfo) :
			CurrentStage(0)
		{
			FMemory::Memzero(this, sizeof(FParticleEditorPromotionTest));

			TestExecutionInfo = InExecutionInfo;

			//Add test stage functions to array in order of execution
			TestStages.Add(&ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest::ContentBrowser_CreateAParticleSystem_Part1);
			TestStages.Add(&ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest::ContentBrowser_CreateAParticleSystem_Part2);
		}
		/**
		* Runs the current test stage
		*/
		bool Update()
		{
			bool bStageComplete = (this->*TestStages[CurrentStage])();
			if (bStageComplete)
			{
				CurrentStage++;
			}
			return CurrentStage >= TestStages.Num();
		}

	private:

		/**
		* ContentBrowser Test Stage: Creating a particle system (Part 1)
		*    Creates a new particle system and opens the cascade editor
		*/
		bool ContentBrowser_CreateAParticleSystem_Part1()
		{
			// @TODO: This should open an existing particle in Cascade; creation should be a different test
			// @TODO: If saving is unnecessary, maybe do edits on default particle, undo (extra testing!), then close without saving
			//Create a Particle system
			UParticleSystemFactoryNew* PSFactory = NewObject<UParticleSystemFactoryNew>();
			const FString& PSName(TEXT("PROMO_ParticleSystem"));
			CreatedPS = Cast<UParticleSystem>(ParticleEditorPromotionTestUtils::CreateAsset(PSFactory, UParticleSystem::StaticClass(), PSName));
			if (CreatedPS)
			{
				// @TODO: Add verification that this completed successfully.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CreatedPS);
				UE_LOGF(LogParticleEditorPromotionTests, Display, "Opened the cascade editor");
			}
			else
			{
				UE_LOGF(LogParticleEditorPromotionTests, Error, "Failed to create a new ParticleSystem");
			}

			return true;
		}

		/**
		* ContentBrowser Test Stage: Creating a particle system (Part 2)
		*    Finds and modifies the StartSize of the particle system, saves the asset, and then closes the editor
		*/
		bool ContentBrowser_CreateAParticleSystem_Part2()
		{
			// @TODO: This should create, edit, and remove a temp asset instead of relying on a previously-created one
			if (CreatedPS)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(CreatedPS, true);

				bool bModifiedSize = false;
				UParticleLODLevel* DefaultLOD = CreatedPS->Emitters[0]->LODLevels[0];
				for (int32 i = 0; i < DefaultLOD->Modules.Num(); ++i)
				{
					UParticleModuleSize* SizeModule = Cast<UParticleModuleSize>(DefaultLOD->Modules[i]);
					if (SizeModule)
					{
						UDistributionVectorUniform* Distribution = Cast<UDistributionVectorUniform>(SizeModule->StartSize.Distribution);
						FEditorPromotionTestUtilities::SetPropertyByName(Distribution, TEXT("Max"), TEXT("(X=100,Y=100,Z=100)"));
						FEditorPromotionTestUtilities::SetPropertyByName(Distribution, TEXT("Min"), TEXT("(X=100,Y=100,Z=100)"));
						bModifiedSize = true;
					}
				}

				if (bModifiedSize)
				{
					UE_LOGF(LogParticleEditorPromotionTests, Display, "Modified ParticleSystem StartSize (Min and Max)");
					// @TODO: Do this with actual editor instead of simulated keyboard commands
					ParticleEditorPromotionTestUtils::SendSaveCascadeCommand();
					// @TODO: Is saving necessary? If so, do we save all assets created during test? If not, why not?
					// @TODO: If we are going to save, verify saving succeeded before counting as a pass
					UE_LOGF(LogParticleEditorPromotionTests, Display, "Saved the particle system");
				}
				else
				{
					UE_LOGF(LogParticleEditorPromotionTests, Error, "Failed to modify ParticleSystem StartSize");
				}

				// @TODO: Close only Cascade instead of all editors
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();
				UE_LOGF(LogParticleEditorPromotionTests, Display, "Closed the cascade editor");
			}
			// @TODO: Return false if test failed after removing sequential-testing setup
			return true;
		}
	};
}


/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FRunParticleEditorPromotionTestCommand, TSharedPtr<ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest>, ParticleEditorPromotionTest);
bool FRunParticleEditorPromotionTestCommand::Update()
{
	return ParticleEditorPromotionTest->Update();
}


/**
* Automation test that handles the build promotion process
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FParticleEditorPromotionTest, "System.Promotion.Editor.Particle Editor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FParticleEditorPromotionTest::RunTest(const FString& Parameters)
{
	TSharedPtr<ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest> ParticleEditorPromotionTest = MakeShareable(new ParticleEditorPromotionTestHelper::FParticleEditorPromotionTest(&ExecutionInfo));
	ADD_LATENT_AUTOMATION_COMMAND(FRunParticleEditorPromotionTestCommand(ParticleEditorPromotionTest));
	return true;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
