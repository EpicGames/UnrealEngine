// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"

class IAssetTypeActions;
class FCascade;
class ICascade;
class IToolkitHost;
class UParticleSystem;

extern const FName CascadeAppIdentifier;

/*-----------------------------------------------------------------------------
   ICascadeModule
-----------------------------------------------------------------------------*/

class ICascadeModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new Cascade instance */
	virtual TSharedRef<ICascade> CreateCascade(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UParticleSystem* ParticleSystem) = 0;

	/** Removes the specified instance from the list of open Cascade toolkits */
	virtual void CascadeClosed(FCascade* CascadeInstance) = 0;

	/** Refreshes the toolkit inspecting the specified particle system */
	virtual void RefreshCascade(UParticleSystem* ParticleSystem) = 0;

	/** Converts all the modules in the specified particle system to seeded modules */
	virtual void ConvertModulesToSeeded(UParticleSystem* ParticleSystem) = 0;
};
